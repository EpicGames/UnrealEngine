{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Blueprint C++ Header Preview",
	"Description": "A tool to help convert Blueprint Classes to Native C++.",
	"Category": "Blueprints",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": true,
	"CanContainContent": true,
	"CanContainVerse": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "BlueprintHeaderView",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	]
}