// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class BlueprintHeaderView : ModuleRules
{
	public BlueprintHeaderView(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
			}
			);
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
				"InputCore",
				"Slate",
				"SlateCore",
				"EditorStyle",
				"UnrealEd",
				"BlueprintGraph",
				"ApplicationCore",
				"DeveloperSettings",
				"WorkspaceMenuStructure",
				"ToolMenus",
				"AssetTools",
			}
			);
		
	}
}
