// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintHeaderView.h"
#include "Brushes/SlateImageBrush.h"
#include "WorkspaceMenuStructure.h"
#include "Misc/Paths.h"
#include "WorkspaceMenuStructureModule.h"
#include "ToolMenu.h"
#include "Widgets/Docking/SDockTab.h"
#include "SBlueprintHeaderView.h"
#include "ContentBrowserModule.h"
#include "Engine/Blueprint.h"
#include "StructUtils/UserDefinedStruct.h"
#include "ToolMenuSection.h"
#include "ToolMenus.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "Toolkits/AssetEditorToolkitMenuContext.h"
#include "BlueprintHeaderViewSettings.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"

#define LOCTEXT_NAMESPACE "BlueprintHeaderViewApp"

namespace BlueprintHeaderViewModule
{
	static const FName HeaderViewTabName = "BlueprintHeaderViewApp";

	TSharedRef<SDockTab> CreateHeaderViewTab(const FSpawnTabArgs& Args)
	{
		return SNew(SDockTab)
			.TabRole(ETabRole::NomadTab)
			[
				SNew(SBlueprintHeaderView)
			];
	}
}

FTextBlockStyle FBlueprintHeaderViewModule::HeaderViewTextStyle;
FTableRowStyle FBlueprintHeaderViewModule::HeaderViewTableRowStyle;
TSharedPtr<FSlateStyleSet> FBlueprintHeaderViewModule::HeaderViewStyleSet;

void FBlueprintHeaderViewModule::StartupModule()
{
	const FVector2D Icon16x16 = FVector2D(16.0);
	const FString PluginContentDir = FPaths::ConvertRelativePathToFull(FPaths::Combine(FPaths::EnginePluginsDir(), TEXT("Editor/BlueprintHeaderView/Content")));
	HeaderViewStyleSet = MakeShared<FSlateStyleSet>("HeaderViewStyle");
	HeaderViewStyleSet->SetContentRoot(PluginContentDir);
	HeaderViewStyleSet->Set("Icons.HeaderView", new FSlateVectorImageBrush(HeaderViewStyleSet->RootToContentDir("BlueprintHeader_16", TEXT(".svg")), Icon16x16));
	FSlateStyleRegistry::RegisterSlateStyle(*HeaderViewStyleSet);

	FGlobalTabmanager::Get()->RegisterNomadTabSpawner(BlueprintHeaderViewModule::HeaderViewTabName, FOnSpawnTab::CreateStatic(&BlueprintHeaderViewModule::CreateHeaderViewTab))
		.SetDisplayName(LOCTEXT("TabTitle", "C++ Header Preview"))
		.SetTooltipText(LOCTEXT("TooltipText", "Displays a Blueprint Class in C++ Header format."))
		.SetGroup(WorkspaceMenu::GetMenuStructure().GetToolsCategory())
		.SetIcon(FSlateIcon(HeaderViewStyleSet->GetStyleSetName(), "Icons.HeaderView"));

	HeaderViewTextStyle = FTextBlockStyle()
		.SetFont(FCoreStyle::GetDefaultFontStyle("Mono", GetDefault<UBlueprintHeaderViewSettings>()->FontSize))
		.SetColorAndOpacity(FSlateColor::UseForeground());

	HeaderViewTableRowStyle = FCoreStyle::Get().GetWidgetStyle<FTableRowStyle>("TableView.Row");
	HeaderViewTableRowStyle.ActiveBrush.TintColor = GetDefault<UBlueprintHeaderViewSettings>()->SelectionColor;
	HeaderViewTableRowStyle.ActiveHoveredBrush.TintColor = GetDefault<UBlueprintHeaderViewSettings>()->SelectionColor;

	SetupAssetEditorMenuExtender();
	SetupContentBrowserContextMenuExtender();
}

void FBlueprintHeaderViewModule::ShutdownModule()
{
	if (ContentBrowserExtenderDelegateHandle.IsValid())
	{
		FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
		TArray< FContentBrowserMenuExtender_SelectedAssets >& CBMenuExtenderDelegates = ContentBrowserModule.GetAllAssetViewContextMenuExtenders();
		CBMenuExtenderDelegates.RemoveAll([ContentBrowserExtenderDelegateHandle=ContentBrowserExtenderDelegateHandle](const FContentBrowserMenuExtender_SelectedAssets& Delegate)
			{
				return Delegate.GetHandle() == ContentBrowserExtenderDelegateHandle;
			});
	}

	FSlateStyleRegistry::UnRegisterSlateStyle(*HeaderViewStyleSet);
	HeaderViewStyleSet.Reset();
}
	
bool FBlueprintHeaderViewModule::IsClassHeaderViewSupported(const UClass* InClass)
{
	return InClass == UBlueprint::StaticClass() || InClass == UUserDefinedStruct::StaticClass();
}

void FBlueprintHeaderViewModule::OpenHeaderViewForAsset(FAssetData InAssetData)
{
	TSharedPtr<SDockTab> HeaderViewTab = FGlobalTabmanager::Get()->TryInvokeTab(BlueprintHeaderViewModule::HeaderViewTabName);

	if (HeaderViewTab.IsValid())
	{
		TSharedRef<SWidget> HeaderViewContentWidget = HeaderViewTab->GetContent();
		StaticCastSharedRef<SBlueprintHeaderView>(HeaderViewContentWidget)->OnAssetSelected(InAssetData);
	}
}

void FBlueprintHeaderViewModule::SetupAssetEditorMenuExtender()
{
	UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("MainFrame.MainMenu.Asset");
	FToolMenuSection& Section = Menu->FindOrAddSection("HeaderViewActions");
	FToolMenuEntry& Entry = Section.AddDynamicEntry("BlueprintHeaderViewCommands", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
		{
			UAssetEditorToolkitMenuContext* MenuContext = InSection.FindContext<UAssetEditorToolkitMenuContext>();
			if (MenuContext && MenuContext->Toolkit.IsValid())
			{
				TSharedPtr<FAssetEditorToolkit> Toolkit = MenuContext->Toolkit.Pin();
				if (Toolkit->IsActuallyAnAsset() && Toolkit->GetObjectsCurrentlyBeingEdited()->Num() == 1)
				{
					for (const UObject* EditedObject : *Toolkit->GetObjectsCurrentlyBeingEdited())
					{
						if (IsClassHeaderViewSupported(EditedObject->GetClass()))
						{
							FAssetData BlueprintAssetData(EditedObject);

							InSection.AddMenuEntry(
								FName("OpenHeaderView"),
								LOCTEXT("OpenAssetHeaderView", "Preview Equivalent C++ Header"),
								LOCTEXT("OpenAssetHeaderViewTooltip", "Provides a preview of what this class could look like in C++"),
								FSlateIcon(HeaderViewStyleSet->GetStyleSetName(), "Icons.HeaderView"),
								FUIAction(FExecuteAction::CreateStatic(&FBlueprintHeaderViewModule::OpenHeaderViewForAsset, BlueprintAssetData))
							);
						}
					}
				}
			}
		}));
}

void FBlueprintHeaderViewModule::SetupContentBrowserContextMenuExtender()
{
	FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	TArray< FContentBrowserMenuExtender_SelectedAssets >& CBMenuExtenderDelegates = ContentBrowserModule.GetAllAssetViewContextMenuExtenders();
	CBMenuExtenderDelegates.Add(FContentBrowserMenuExtender_SelectedAssets::CreateStatic(&FBlueprintHeaderViewModule::OnExtendContentBrowserAssetSelectionMenu));
	ContentBrowserExtenderDelegateHandle = CBMenuExtenderDelegates.Last().GetHandle();
}

TSharedRef<FExtender> FBlueprintHeaderViewModule::OnExtendContentBrowserAssetSelectionMenu(const TArray<FAssetData>& SelectedAssets)
{
	TSharedRef<FExtender> Extender = MakeShared<FExtender>();

	if (SelectedAssets.Num() == 1)
	{
		if (IsClassHeaderViewSupported(SelectedAssets[0].GetClass()))
		{
			Extender->AddMenuExtension("GetAssetActions", EExtensionHook::After, nullptr, FMenuExtensionDelegate::CreateLambda(
				[SelectedAssets](FMenuBuilder& MenuBuilder) {
					MenuBuilder.AddMenuEntry(
						LOCTEXT("OpenHeaderView", "Preview Equivalent C++ Header"),
						LOCTEXT("OpenHeaderViewTooltip", "Provides a preview of what this class could look like in C++"),
						FSlateIcon(HeaderViewStyleSet->GetStyleSetName(), "Icons.HeaderView"),
						FUIAction(FExecuteAction::CreateStatic(&FBlueprintHeaderViewModule::OpenHeaderViewForAsset, SelectedAssets[0]))
						);
				})
			);
		}
	}

	return Extender;
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FBlueprintHeaderViewModule, BlueprintHeaderView)
