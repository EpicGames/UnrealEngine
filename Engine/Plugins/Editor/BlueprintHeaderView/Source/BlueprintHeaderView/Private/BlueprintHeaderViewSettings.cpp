// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintHeaderViewSettings.h"
#include "BlueprintHeaderView.h"
#include "Styling/SlateTypes.h"
#include "Styling/StyleColors.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlueprintHeaderViewSettings)

FHeaderViewSyntaxColors::FHeaderViewSyntaxColors()
	: Comment(FStyleColors::AccentGreen.GetSpecifiedColor())
	, Error(FStyleColors::Error.GetSpecifiedColor())
	, Macro(FStyleColors::AccentPurple.GetSpecifiedColor())
	, Typename(0.0f, 0.3f, 0.3f)
	, Identifier(FStyleColors::White.GetSpecifiedColor())
	, Keyword(FStyleColors::AccentBlue.GetSpecifiedColor())
{

}

UBlueprintHeaderViewSettings::UBlueprintHeaderViewSettings()
{
	
}

FName UBlueprintHeaderViewSettings::GetCategoryName() const
{
	return TEXT("Plugins");
}

FText UBlueprintHeaderViewSettings::GetSectionText() const
{
	return NSLOCTEXT("BlueprintHeaderViewSettings", "HeaderViewSectionText", "Blueprint Header View");
}

FName UBlueprintHeaderViewSettings::GetSectionName() const
{
	return TEXT("Blueprint Header View");
}

void UBlueprintHeaderViewSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (!PropertyChangedEvent.MemberProperty)
	{
		return;
	}

	const FName PropertyChangedName = PropertyChangedEvent.MemberProperty->GetFName();

	if (PropertyChangedName == GET_MEMBER_NAME_CHECKED(UBlueprintHeaderViewSettings, FontSize))
	{
		FBlueprintHeaderViewModule::HeaderViewTextStyle.SetFontSize(FontSize);
	}

	if (PropertyChangedName == GET_MEMBER_NAME_CHECKED(UBlueprintHeaderViewSettings, SelectionColor))
	{
		FBlueprintHeaderViewModule::HeaderViewTableRowStyle.ActiveBrush.TintColor = SelectionColor;
		FBlueprintHeaderViewModule::HeaderViewTableRowStyle.ActiveHoveredBrush.TintColor = SelectionColor;
	}
}

