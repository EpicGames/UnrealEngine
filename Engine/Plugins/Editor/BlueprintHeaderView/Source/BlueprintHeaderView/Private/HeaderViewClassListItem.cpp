// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeaderViewClassListItem.h"
#include "IAssetTools.h"
#include "String/LineEndings.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "AssetViewUtils.h"
#include "AssetToolsModule.h"

#define LOCTEXT_NAMESPACE "FHeaderViewClassListItem"

FHeaderViewListItemPtr FHeaderViewClassListItem::Create(TWeakObjectPtr<UBlueprint> InBlueprint)
{
	return MakeShareable(new FHeaderViewClassListItem(InBlueprint));
}

void FHeaderViewClassListItem::ExtendContextMenu(FMenuBuilder& InMenuBuilder, TWeakObjectPtr<UObject> InAsset)
{
	if (!bIsValidName)
	{
		if (UBlueprint* Blueprint = Cast<UBlueprint>(InAsset.Get()))
		{
			TWeakObjectPtr<UBlueprint> WeakBlueprint = Blueprint;
			InMenuBuilder.AddVerifiedEditableText(LOCTEXT("RenameBlueprint", "Rename Blueprint"),
				LOCTEXT("RenameItemTooltip", "Renames this Blueprint\nThis Blueprint name is not a legal C++ identifier."),
				FSlateIcon(),
				FText::FromString(Blueprint->GetName()),
				FOnVerifyTextChanged::CreateSP(this, &FHeaderViewClassListItem::OnVerifyRenameTextChanged, WeakBlueprint),
				FOnTextCommitted::CreateSP(this, &FHeaderViewClassListItem::OnRenameTextComitted, WeakBlueprint)
			);
		}
	}
}

FString FHeaderViewClassListItem::GetConditionalUClassSpecifiers(const UBlueprint* Blueprint) const
{
	TStringBuilder<256> AdditionalSpecifiers;

	if (Blueprint->bGenerateConstClass)
	{
		AdditionalSpecifiers.Append(TEXT(", Const"));
	}

	if (Blueprint->bGenerateAbstractClass)
	{
		AdditionalSpecifiers.Append(TEXT(", Abstract"));
	}

	if (!Blueprint->BlueprintCategory.IsEmpty())
	{
		AdditionalSpecifiers.Append(FString::Printf(TEXT(", Category=\"%s\""), *Blueprint->BlueprintCategory));
	}

	if (!Blueprint->HideCategories.IsEmpty())
	{
		AdditionalSpecifiers.Append(TEXT(", HideCategories=("));

		for (int32 HideCategoryIdx = 0; HideCategoryIdx < Blueprint->HideCategories.Num(); ++HideCategoryIdx)
		{
			if (HideCategoryIdx == 0)
			{
				AdditionalSpecifiers.Append(FString::Printf(TEXT("\"%s\""), *Blueprint->HideCategories[HideCategoryIdx]));
			}
			else
			{
				AdditionalSpecifiers.Append(FString::Printf(TEXT(", \"%s\""), *Blueprint->HideCategories[HideCategoryIdx]));
			}
		}

		AdditionalSpecifiers.Append(TEXT(")"));
	}

	if (!Blueprint->BlueprintDisplayName.IsEmpty() || !Blueprint->BlueprintNamespace.IsEmpty())
	{
		AdditionalSpecifiers.Append(TEXT(", meta=("));

		int32 SpecifierCount = 0;

		if (!Blueprint->BlueprintDisplayName.IsEmpty())
		{
			AdditionalSpecifiers.Append(FString::Printf(TEXT("DisplayName=\"%s\""), *Blueprint->BlueprintDisplayName));
			++SpecifierCount;
		}

		if (!Blueprint->BlueprintNamespace.IsEmpty())
		{
			if (SpecifierCount > 0)
			{
				AdditionalSpecifiers.Append(FString::Printf(TEXT(", Namespace=\"%s\""), *Blueprint->BlueprintNamespace));
			}
			else
			{
				AdditionalSpecifiers.Append(FString::Printf(TEXT("Namespace=\"%s\""), *Blueprint->BlueprintNamespace));
			}
		}

		AdditionalSpecifiers.Append(TEXT(")"));
	}

	return AdditionalSpecifiers.ToString();
}

FHeaderViewClassListItem::FHeaderViewClassListItem(TWeakObjectPtr<UBlueprint> InBlueprint)
{
	if (UBlueprint* Blueprint = InBlueprint.Get())
	{
		// Avoid lots of reallocations
		RawItemString.Reserve(512);
		RichTextString.Reserve(512);

		// Format comment
		{
			FString Comment = Blueprint->BlueprintDescription.IsEmpty() ? TEXT("Please add a class description") : Blueprint->BlueprintDescription;
			FormatCommentString(Comment, RawItemString, RichTextString);
		}

		// Add the UCLASS specifiers
		// i.e. UCLASS(Blueprintable, BlueprintType, Category=BlueprintCategory)
		{
			FString AdditionalSpecifiers = GetConditionalUClassSpecifiers(Blueprint);

			// Always add Blueprintable and BlueprintType
			RawItemString += FString::Printf(TEXT("\nUCLASS(Blueprintable, BlueprintType%s)"), *AdditionalSpecifiers);
			RichTextString += FString::Printf(TEXT("\n<%s>UCLASS</>(Blueprintable, BlueprintType%s)"), *HeaderViewSyntaxDecorators::MacroDecorator, *AdditionalSpecifiers);
		}

		// Add the class declaration line
		// i.e. class ClassName : public ParentClass
		{
			const FString BlueprintName = Blueprint->SkeletonGeneratedClass->GetPrefixCPP() + Blueprint->GetName();

			const UObject* ParentBlueprint = Blueprint->ParentClass->ClassGeneratedBy;
			FString ParentClassName = FString::Printf(TEXT("%s%s"), 
				ParentBlueprint ? TEXT("") : Blueprint->ParentClass->GetPrefixCPP(),
				ParentBlueprint ? *ParentBlueprint->GetName() : *Blueprint->ParentClass->GetAuthoredName());

			bIsValidName = IsValidCPPIdentifier(BlueprintName);
			const FString& NameDecorator = bIsValidName ? HeaderViewSyntaxDecorators::TypenameDecorator : HeaderViewSyntaxDecorators::ErrorDecorator;

			RawItemString += FString::Printf(TEXT("\nclass %s : public %s\n{\n\tGENERATED_BODY()"), *BlueprintName, *ParentClassName);
			RichTextString += FString::Printf(TEXT("\n<%s>class</> <%s>%s</> : <%s>public</> <%s>%s</>\n{\n\t<%s>GENERATED_BODY</>()"), 
				*HeaderViewSyntaxDecorators::KeywordDecorator,
				*NameDecorator, 
				*BlueprintName,
				*HeaderViewSyntaxDecorators::KeywordDecorator,
				*HeaderViewSyntaxDecorators::TypenameDecorator,
				*ParentClassName,
				*HeaderViewSyntaxDecorators::MacroDecorator
			);
		}

		// normalize to platform newlines
		UE::String::ToHostLineEndingsInline(RawItemString);
		UE::String::ToHostLineEndingsInline(RichTextString);
	}
}

FString FHeaderViewClassListItem::GetRenamedBlueprintPath(const UBlueprint* Blueprint, const FString& NewName) const
{
	check(Blueprint);
	FString NewPath = Blueprint->GetPathName();
	int32 Index = 0;
	if (!ensure(NewPath.FindLastChar(TEXT('/'), Index)))
	{
		// AssetViewUtils::IsValidObjectPathForCreate will return false for an empty string
		return TEXT("");
	}
	NewPath.LeftInline(Index);
	NewPath /= FString::Printf(TEXT("%s.%s"), *NewName, *NewName);
	return NewPath;
}

bool FHeaderViewClassListItem::OnVerifyRenameTextChanged(const FText& InNewName, FText& OutErrorText, TWeakObjectPtr<UBlueprint> InBlueprint)
{
	if (const UBlueprint* Blueprint = InBlueprint.Get())
	{
		const FString NewPath = GetRenamedBlueprintPath(Blueprint, InNewName.ToString());
		if (!AssetViewUtils::IsValidObjectPathForCreate(NewPath, Blueprint->GetClass(), OutErrorText))
		{
			return false;
		}

		if (!IsValidCPPIdentifier(InNewName.ToString()))
		{
			OutErrorText = InvalidCPPIdentifierErrorText;
			return false;
		}

		return true;
	}

	return false;
}

void FHeaderViewClassListItem::OnRenameTextComitted(const FText& CommittedText, ETextCommit::Type TextCommitType, TWeakObjectPtr<UBlueprint> InBlueprint)
{
	if (TextCommitType == ETextCommit::OnEnter)
	{
		FText ErrorText;
		if (OnVerifyRenameTextChanged(CommittedText, ErrorText, InBlueprint))
		{
			if (UBlueprint* Blueprint = InBlueprint.Get())
			{
				const FString NewPath = GetRenamedBlueprintPath(Blueprint, CommittedText.ToString());
				FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
				TArray<FAssetRenameData> AssetToRename = { FAssetRenameData(Blueprint, NewPath) };
				AssetToolsModule.Get().RenameAssets(AssetToRename);
				FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
