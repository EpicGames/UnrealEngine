// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SBlueprintHeaderView.h"

namespace ETextCommit { enum Type : int; }

class FMenuBuilder;

/** A header view list item that displays the class declaration */
struct FHeaderViewClassListItem : public FHeaderViewListItem
{
public:
	/** Creates a list item for the Header view representing a class declaration for the given blueprint */
	static FHeaderViewListItemPtr Create(TWeakObjectPtr<UBlueprint> InBlueprint);

	//~ FHeaderViewListItem Interface
	virtual void ExtendContextMenu(FMenuBuilder& InMenuBuilder, TWeakObjectPtr<UObject> InAsset) override;
	//~ End FHeaderViewListItem Interface
protected:
	FString GetConditionalUClassSpecifiers(const UBlueprint* Blueprint) const;

	FHeaderViewClassListItem(TWeakObjectPtr<UBlueprint> InBlueprint);

	FString GetRenamedBlueprintPath(const UBlueprint* Blueprint, const FString& NewName) const;

	bool OnVerifyRenameTextChanged(const FText& InNewName, FText& OutErrorText, TWeakObjectPtr<UBlueprint> InBlueprint);

	void OnRenameTextComitted(const FText& CommittedText, ETextCommit::Type TextCommitType, TWeakObjectPtr<UBlueprint> InBlueprint);

protected:
	/** Whether this class name is valid C++ (no spaces, special chars, etc) */
	bool bIsValidName = true;
};
