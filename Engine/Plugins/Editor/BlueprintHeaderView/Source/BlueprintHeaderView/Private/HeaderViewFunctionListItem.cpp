// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeaderViewFunctionListItem.h"
#include "Kismet2/Kismet2NameValidators.h"
#include "String/LineEndings.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "K2Node_FunctionEntry.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Kismet2/KismetEditorUtilities.h"

#define LOCTEXT_NAMESPACE "FHeaderViewFunctionListItem"

FHeaderViewListItemPtr FHeaderViewFunctionListItem::Create(const UK2Node_FunctionEntry* FunctionEntry)
{
	return MakeShareable(new FHeaderViewFunctionListItem(FunctionEntry));
}

void FHeaderViewFunctionListItem::ExtendContextMenu(FMenuBuilder& InMenuBuilder, TWeakObjectPtr<UObject> InAsset)
{
	if (UBlueprint* Blueprint = Cast<UBlueprint>(InAsset.Get()))
	{
		TWeakObjectPtr<UBlueprint> WeakBlueprint = Blueprint;
		InMenuBuilder.AddMenuEntry(LOCTEXT("JumpToDefinition", "Jump to Definition"),
			LOCTEXT("JumpToDefinitionTooltip", "Opens this function in the Blueprint Editor"),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &FHeaderViewFunctionListItem::JumpToDefinition, WeakBlueprint))
		);

		if (!IllegalName.IsNone())
		{
			InMenuBuilder.AddVerifiedEditableText(LOCTEXT("RenameItem", "Rename Function"),
				LOCTEXT("RenameItemTooltip", "Renames this function in the Blueprint\nThis function name is not a legal C++ identifier."),
				FSlateIcon(),
				FText::FromName(IllegalName),
				FOnVerifyTextChanged::CreateSP(this, &FHeaderViewFunctionListItem::OnVerifyRenameFunctionTextChanged, WeakBlueprint),
				FOnTextCommitted::CreateSP(this, &FHeaderViewFunctionListItem::OnRenameFunctionTextCommitted, WeakBlueprint, GraphName)
			);
		}

		for (FName IllegalParam : IllegalParameters)
		{
			InMenuBuilder.AddVerifiedEditableText(LOCTEXT("RenameParm", "Rename Parameter"),
				LOCTEXT("RenameParmTooltip", "Renames this function parameter in the Blueprint\nThis parameter name is not a legal C++ identifier."),
				FSlateIcon(),
				FText::FromName(IllegalParam),
				FOnVerifyTextChanged::CreateSP(this, &FHeaderViewFunctionListItem::OnVerifyRenameParameterTextChanged, WeakBlueprint, GraphName),
				FOnTextCommitted::CreateSP(this, &FHeaderViewFunctionListItem::OnRenameParameterTextCommitted, WeakBlueprint, GraphName, IllegalParam)
			);
		}
	}
}

void FHeaderViewFunctionListItem::OnMouseButtonDoubleClick(TWeakObjectPtr<UObject> InAsset)
{
	TWeakObjectPtr<UBlueprint> WeakBlueprint = Cast<UBlueprint>(InAsset.Get());
	JumpToDefinition(WeakBlueprint);
}

FString FHeaderViewFunctionListItem::GetConditionalUFunctionSpecifiers(const UFunction* SigFunction) const
{
	check(SigFunction);

	TStringBuilder<256> AdditionalSpecifiers;

	if (SigFunction->FunctionFlags & FUNC_BlueprintPure)
	{
		AdditionalSpecifiers.Append(TEXT("BlueprintPure"));
	}
	else
	{
		AdditionalSpecifiers.Append(TEXT("BlueprintCallable"));
	}

	if (SigFunction->GetBoolMetaData(FBlueprintMetadata::MD_CallInEditor))
	{
		AdditionalSpecifiers.Append(TEXT(", CallInEditor"));
	}

	if (SigFunction->FunctionFlags & FUNC_Exec)
	{
		AdditionalSpecifiers.Append(TEXT(", Exec"));
	}

	const FString& Category = SigFunction->GetMetaData(FBlueprintMetadata::MD_FunctionCategory);
	if (!Category.IsEmpty())
	{
		AdditionalSpecifiers.Append(FString::Printf(TEXT(", Category=\"%s\""), *Category));
	}

	// Meta Specifiers
	{
		TArray<FString> MetaSpecifiers;

		if (SigFunction->GetBoolMetaData(FBlueprintMetadata::MD_ThreadSafe))
		{
			MetaSpecifiers.Emplace(TEXT("BlueprintThreadSafe"));
		}

		const FString& CompactNodeTitle = SigFunction->GetMetaData(FBlueprintMetadata::MD_CompactNodeTitle);
		if (!CompactNodeTitle.IsEmpty())
		{
			MetaSpecifiers.Emplace(FString::Printf(TEXT("CompactNodeTitle=\"%s\""), *CompactNodeTitle));
		}

		const FString& DisplayName = SigFunction->GetMetaData(FBlueprintMetadata::MD_DisplayName);
		if (!DisplayName.IsEmpty())
		{
			MetaSpecifiers.Emplace(FString::Printf(TEXT("DisplayName=\"%s\""), *DisplayName));
		}
		
		const FString& Keywords = SigFunction->GetMetaData(FBlueprintMetadata::MD_FunctionKeywords);
		if (!Keywords.IsEmpty())
		{
			MetaSpecifiers.Emplace(FString::Printf(TEXT("Keywords=\"%s\""), *Keywords));
		}

		if (!MetaSpecifiers.IsEmpty())
		{
			AdditionalSpecifiers.Append(FString::Printf(TEXT(", meta=(%s)"), *FString::Join(MetaSpecifiers, TEXT(", "))));
		}
	}

	return AdditionalSpecifiers.ToString();
}

void FHeaderViewFunctionListItem::AppendFunctionParameters(const UFunction* SignatureFunction)
{
	check(SignatureFunction);

	TArray<FString> Parameters;

	// cache the return property to check against later
	const FProperty* ReturnProperty = SignatureFunction->GetReturnProperty();

	int32 ParamIdx = 0;
	for (TFieldIterator<const FProperty> ParmIt(SignatureFunction); ParmIt; ++ParmIt)
	{
		// ReturnValue property shouldn't be duplicated in the parameter list
		if (*ParmIt == ReturnProperty)
		{
			continue;
		}

		if (ParamIdx > 0)
		{
			RawItemString.Append(TEXT(", "));
			RichTextString.Append(TEXT(", "));
		}

		if (ParmIt->HasAnyPropertyFlags(CPF_ConstParm))
		{
			RawItemString.Append(TEXT("const "));
			RichTextString.Append(FString::Printf(TEXT("<%s>const</> "), *HeaderViewSyntaxDecorators::KeywordDecorator));
		}
		// If a param is declared as const&, then it is already treated as input and UPARAM(ref) will just be clutter
		else if (ParmIt->HasAnyPropertyFlags(CPF_ReferenceParm))
		{
			RawItemString.Append(TEXT("UPARAM(ref) "));
			RichTextString.Append(FString::Printf(TEXT("<%s>UPARAM</>(ref) "), *HeaderViewSyntaxDecorators::MacroDecorator));
		}

		const FString Typename = GetCPPTypenameForProperty(*ParmIt);
		const FString ParmName = ParmIt->GetAuthoredName();
		
		const bool bIsValidName = IsValidCPPIdentifier(ParmName);
		const FString* IdentifierDecorator = &HeaderViewSyntaxDecorators::IdentifierDecorator;
		if (!bIsValidName)
		{
			IllegalParameters.Emplace(ParmName);
			IdentifierDecorator = &HeaderViewSyntaxDecorators::ErrorDecorator;
		}

		if (ParmIt->HasAnyPropertyFlags(CPF_OutParm | CPF_ReferenceParm))
		{
			RawItemString.Append(FString::Printf(TEXT("%s& %s"), *Typename, *ParmName));
			RichTextString.Append(FString::Printf(TEXT("<%s>%s</>& <%s>%s</>"), *HeaderViewSyntaxDecorators::TypenameDecorator, *Typename, **IdentifierDecorator, *ParmName));
		}
		else
		{
			RawItemString.Append(FString::Printf(TEXT("%s %s"), *Typename, *ParmName));
			RichTextString.Append(FString::Printf(TEXT("<%s>%s</> <%s>%s</>"), *HeaderViewSyntaxDecorators::TypenameDecorator, *Typename, **IdentifierDecorator, *ParmName));
		}

		++ParamIdx;
	}
}

FHeaderViewFunctionListItem::FHeaderViewFunctionListItem(const UK2Node_FunctionEntry* FunctionEntry)
{
	check(FunctionEntry);

	GraphName = FunctionEntry->GetGraph()->GetFName();
	RawItemString.Reserve(512);
	RichTextString.Reserve(512);

	if (const UFunction* ResolvedFunc = FunctionEntry->FindSignatureFunction())
	{
		// Format comment
		{
			FString Comment = ResolvedFunc->GetMetaData(FBlueprintMetadata::MD_Tooltip);
			if (Comment.IsEmpty())
			{
				Comment = TEXT("Please add a function description");
			}

			if (ResolvedFunc->HasAnyFunctionFlags(FUNC_Event))
			{
				const UClass* OriginClass = ResolvedFunc->GetOwnerClass();
				const FString OriginClassName = OriginClass->GetPrefixCPP() + OriginClass->GetName();
				const FString EventType = ResolvedFunc->HasAnyFunctionFlags(FUNC_Native) ? TEXT("BlueprintNativeEvent") : TEXT("BlueprintImplementableEvent");
				Comment.Append(FString::Printf(TEXT("\n\nNOTE: This function is linked to %s: %s::%s"), *EventType, *OriginClassName, *ResolvedFunc->GetName()));
			}

			FormatCommentString(Comment, RawItemString, RichTextString);
		}

		// Add Deprecation message if present
		if (ResolvedFunc->GetBoolMetaData(FBlueprintMetadata::MD_DeprecatedFunction))
		{
			FString DeprecationMessage = ResolvedFunc->GetMetaData(FBlueprintMetadata::MD_DeprecationMessage);
			if (DeprecationMessage.IsEmpty())
			{
				DeprecationMessage = TEXT("Please add a deprecation message.");
			}

			const FString EngineVersionString = FEngineVersion::Current().ToString(EVersionComponent::Patch);

			RawItemString += FString::Printf(TEXT("\nUE_DEPRECATED(%s, \"%s\")"), *EngineVersionString, *DeprecationMessage);
			RichTextString += FString::Printf(TEXT("\n<%s>UE_DEPRECATED</>(%s, \"%s\")"), *HeaderViewSyntaxDecorators::MacroDecorator, *EngineVersionString, *DeprecationMessage);
		}

		// Add the UFUNCTION specifiers
		// i.e. UFUNCTION(BlueprintCallable, Category="Function Category")
		{
			const FString AdditionalSpecifiers = GetConditionalUFunctionSpecifiers(ResolvedFunc);

			// Always add Blueprintable and BlueprintType
			RawItemString += FString::Printf(TEXT("\nUFUNCTION(%s)"), *AdditionalSpecifiers);
			RichTextString += FString::Printf(TEXT("\n<%s>UFUNCTION</>(%s)"), *HeaderViewSyntaxDecorators::MacroDecorator, *AdditionalSpecifiers);
		}

		// Add the function declaration line
		// i.e. void FunctionName(Type InParam1, UPARAM(ref) Type2& InParam2, Type3& OutParam1)
		{
			FString FunctionName;
			if (FunctionEntry->CustomGeneratedFunctionName.IsNone())
			{
				FunctionName = ResolvedFunc->GetName();
			}
			else
			{
				FunctionName = FunctionEntry->CustomGeneratedFunctionName.ToString();
			}

			const bool bValidName = IsValidCPPIdentifier(FunctionName);
			const FString* IdentifierDecorator = &HeaderViewSyntaxDecorators::IdentifierDecorator;
			if (!bValidName)
			{
				IllegalName = ResolvedFunc->GetFName();
				IdentifierDecorator = &HeaderViewSyntaxDecorators::ErrorDecorator;
			}

			if (FProperty* ReturnProperty = ResolvedFunc->GetReturnProperty())
			{
				const FString Typename = ReturnProperty->GetCPPType();
				RawItemString += FString::Printf(TEXT("\n%s %s("), *Typename, *FunctionName);
				RichTextString += FString::Printf(TEXT("\n<%s>%s</> <%s>%s</>("), *HeaderViewSyntaxDecorators::TypenameDecorator, *Typename, **IdentifierDecorator, *FunctionName);
			}
			else
			{
				RawItemString += FString::Printf(TEXT("\nvoid %s("), *FunctionName);
				RichTextString += FString::Printf(TEXT("\n<%s>void</> <%s>%s</>("), *HeaderViewSyntaxDecorators::KeywordDecorator, **IdentifierDecorator, *FunctionName);
			}

			AppendFunctionParameters(ResolvedFunc);

			if (FunctionEntry->GetFunctionFlags() & FUNC_Const)
			{
				RawItemString += TEXT(") const;");
				RichTextString += FString::Printf(TEXT(") <%s>const</>;"), *HeaderViewSyntaxDecorators::KeywordDecorator);
			}
			else
			{
				RawItemString += TEXT(");");
				RichTextString += TEXT(");");
			}
		}

		// indent item
		RawItemString.InsertAt(0, TEXT("\t"));
		RichTextString.InsertAt(0, TEXT("\t"));
		RawItemString.ReplaceInline(TEXT("\n"), TEXT("\n\t"));
		RichTextString.ReplaceInline(TEXT("\n"), TEXT("\n\t"));

		// normalize to platform newlines
		UE::String::ToHostLineEndingsInline(RawItemString);
		UE::String::ToHostLineEndingsInline(RichTextString);
	}
}

void FHeaderViewFunctionListItem::OnRenameFunctionTextCommitted(const FText& CommittedText, ETextCommit::Type TextCommitType, TWeakObjectPtr<UBlueprint> WeakBlueprint, FName OldGraphName)
{
	if (TextCommitType == ETextCommit::OnEnter)
	{
		if (UBlueprint* Blueprint = WeakBlueprint.Get())
		{
			const FString& CommittedString = CommittedText.ToString();
			if (IsValidCPPIdentifier(CommittedString))
			{
				const TObjectPtr<UEdGraph>* FunctionGraph = Blueprint->FunctionGraphs.FindByPredicate([OldGraphName](const TObjectPtr<UEdGraph>& Graph)
					{
						return Graph->GetFName() == OldGraphName;
					});

				if (FunctionGraph)
				{
					FBlueprintEditorUtils::RenameGraph(FunctionGraph->Get(), CommittedString);
				}
				else
				{
					UE_LOGF(LogBlueprintHeaderView, Warning, "Could not find Function Graph named \"%ls\" in Blueprint \"%ls\"", *OldGraphName.ToString(), *Blueprint->GetName());
				}
			}
		}
	}
}

bool FHeaderViewFunctionListItem::OnVerifyRenameFunctionTextChanged(const FText& InNewName, FText& OutErrorText, TWeakObjectPtr<UBlueprint> WeakBlueprint)
{
	if (const UBlueprint* Blueprint = WeakBlueprint.Get())
	{
		if (!IsValidCPPIdentifier(InNewName.ToString()))
		{
			OutErrorText = InvalidCPPIdentifierErrorText;
			return false;
		}

		FKismetNameValidator NameValidator(Blueprint, NAME_None, nullptr);
		const EValidatorResult Result = NameValidator.IsValid(InNewName.ToString());
		if (Result != EValidatorResult::Ok)
		{
			OutErrorText = GetErrorTextFromValidatorResult(Result);
			return false;
		}

		return true;
	}

	return false;
}

bool FHeaderViewFunctionListItem::OnVerifyRenameParameterTextChanged(const FText& InNewName, FText& OutErrorText, TWeakObjectPtr<UBlueprint> WeakBlueprint, FName OldGraphName)
{
	if (const UBlueprint* Blueprint = WeakBlueprint.Get())
	{
		if (!IsValidCPPIdentifier(InNewName.ToString()))
		{
			OutErrorText = InvalidCPPIdentifierErrorText;
			return false;
		}

		const TObjectPtr<UEdGraph>* FunctionGraph = Blueprint->FunctionGraphs.FindByPredicate([OldGraphName](const TObjectPtr<UEdGraph>& Graph)
			{
				return Graph->GetFName() == OldGraphName;
			});

		if (FunctionGraph)
		{
			TArray<UK2Node_FunctionEntry*> Entry;
			FunctionGraph->Get()->GetNodesOfClass<UK2Node_FunctionEntry>(Entry);
			if (ensureMsgf(Entry.Num() == 1 && Entry[0], TEXT("Function Graph \"%s\" in Blueprint \"%s\" does not have exactly one Entry Node"), *FunctionGraph->Get()->GetName(), *Blueprint->GetName()))
			{
				FKismetNameValidator NameValidator(Blueprint, NAME_None, Entry[0]->FindSignatureFunction());
				const EValidatorResult Result = NameValidator.IsValid(InNewName.ToString());
				if (Result != EValidatorResult::Ok)
				{
					OutErrorText = GetErrorTextFromValidatorResult(Result);
					return false;
				}
			}
		}

		return true;
	}

	return false;
}

void FHeaderViewFunctionListItem::OnRenameParameterTextCommitted(const FText& CommittedText, ETextCommit::Type TextCommitType, TWeakObjectPtr<UBlueprint> WeakBlueprint, FName OldGraphName, FName OldParamName)
{
	if (TextCommitType == ETextCommit::OnEnter)
	{
		if (UBlueprint* Blueprint = WeakBlueprint.Get())
		{
			const FString& CommittedString = CommittedText.ToString();
			if (IsValidCPPIdentifier(CommittedString))
			{
				const TObjectPtr<UEdGraph>* FunctionGraph = Blueprint->FunctionGraphs.FindByPredicate([OldGraphName](const TObjectPtr<UEdGraph>& Graph)
					{
						return Graph->GetFName() == OldGraphName;
					});

				if (FunctionGraph)
				{
					TArray<UK2Node_FunctionEntry*> Entry;
					FunctionGraph->Get()->GetNodesOfClass<UK2Node_FunctionEntry>(Entry);
					if (ensureMsgf(Entry.Num() == 1 && Entry[0], TEXT("Function Graph \"%s\" in Blueprint \"%s\" does not have exactly one Entry Node"), *FunctionGraph->Get()->GetName(), *Blueprint->GetName()))
					{
						if (Entry[0]->RenameUserDefinedPin(OldParamName, FName(CommittedString)) == ERenamePinResult_Success)
						{
							FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(Blueprint);
						}
					}
				}
				else
				{
					UE_LOGF(LogBlueprintHeaderView, Warning, "Could not find Function Graph named \"%ls\" in Blueprint \"%ls\"", *OldGraphName.ToString(), *Blueprint->GetName());
				}
			}
		}
	}
}

void FHeaderViewFunctionListItem::JumpToDefinition(TWeakObjectPtr<UBlueprint> WeakBlueprint) const
{
	if (const UBlueprint* Blueprint = WeakBlueprint.Get())
	{
		const TObjectPtr<UEdGraph>* FunctionGraph = Blueprint->FunctionGraphs.FindByPredicate([GraphName=GraphName](const TObjectPtr<UEdGraph>& Graph)
			{
				return Graph->GetFName() == GraphName;
			});

		if (FunctionGraph)
		{
			TArray<UK2Node_FunctionEntry*> Entry;
			FunctionGraph->Get()->GetNodesOfClass<UK2Node_FunctionEntry>(Entry);
			if (ensureMsgf(Entry.Num() == 1 && Entry[0], TEXT("Function Graph \"%s\" in Blueprint \"%s\" does not have exactly one Entry Node"), *FunctionGraph->Get()->GetName(), *Blueprint->GetName()))
			{
				FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(Entry[0]);
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
