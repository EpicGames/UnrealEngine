// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SBlueprintHeaderView.h"

namespace ETextCommit { enum Type : int; }

class UK2Node_FunctionEntry;

/** A header view list item that displays a function declaration */
struct FHeaderViewFunctionListItem : public FHeaderViewListItem
{
	/** Creates a list item for the Header view representing a function declaration for the given blueprint function */
	static FHeaderViewListItemPtr Create(const UK2Node_FunctionEntry* FunctionEntry);

	//~ FHeaderViewListItem Interface
	virtual void ExtendContextMenu(FMenuBuilder& InMenuBuilder, TWeakObjectPtr<UObject> InAsset) override;
	virtual void OnMouseButtonDoubleClick(TWeakObjectPtr<UObject> InAsset) override;
	//~ End FHeaderViewListItem Interface

protected:
	FHeaderViewFunctionListItem(const UK2Node_FunctionEntry* FunctionEntry);

	/** Returns a string containing the specifiers for the UFUNCTION line */
	FString GetConditionalUFunctionSpecifiers(const UFunction* SigFunction) const;

	/** Adds Function parameters to the RichText and PlainText strings */
	void AppendFunctionParameters(const UFunction* SignatureFunction);

	// UI Callback Functions
	bool OnVerifyRenameFunctionTextChanged(const FText& InNewName, FText& OutErrorText, TWeakObjectPtr<UBlueprint> WeakBlueprint);
	bool OnVerifyRenameParameterTextChanged(const FText& InNewName, FText& OutErrorText, TWeakObjectPtr<UBlueprint> WeakBlueprint, FName OldGraphName);
	void OnRenameFunctionTextCommitted(const FText& CommittedText, ETextCommit::Type TextCommitType, TWeakObjectPtr<UBlueprint> WeakBlueprint, FName OldGraphName);
	void OnRenameParameterTextCommitted(const FText& CommittedText, ETextCommit::Type TextCommitType, TWeakObjectPtr<UBlueprint> WeakBlueprint, FName OldGraphName, FName OldParamName);
	void JumpToDefinition(TWeakObjectPtr<UBlueprint> WeakBlueprint) const;
protected:
	/** None if the function name is legal C++, else the name of the function */
	FName IllegalName = NAME_None;

	/** Name of the Function Graph this item represents */
	FName GraphName = NAME_None;

	/** Names of any function parameters that are not legal C++ */
	TArray<FName> IllegalParameters;
};
