// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeaderViewStructListItem.h"
#include "StructUtils/UserDefinedStruct.h"
#include "IAssetTools.h"
#include "String/LineEndings.h"
#include "Kismet2/StructureEditorUtils.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "AssetViewUtils.h"
#include "AssetToolsModule.h"

#define LOCTEXT_NAMESPACE "FHeaderViewStructListItem"

FHeaderViewListItemPtr FHeaderViewStructListItem::Create(TWeakObjectPtr<UUserDefinedStruct> InStruct)
{
	return MakeShareable(new FHeaderViewStructListItem(InStruct));
}

void FHeaderViewStructListItem::ExtendContextMenu(FMenuBuilder& InMenuBuilder, TWeakObjectPtr<UObject> InAsset)
{
	if (!bIsValidName)
	{
		if (UUserDefinedStruct* Struct = Cast<UUserDefinedStruct>(InAsset.Get()))
		{
			TWeakObjectPtr<UUserDefinedStruct> WeakStruct = Struct;
			InMenuBuilder.AddVerifiedEditableText(LOCTEXT("RenameStruct", "Rename Struct"),
				LOCTEXT("RenameItemTooltip", "Renames this Struct\nThis Struct name is not a legal C++ identifier."),
				FSlateIcon(),
				FText::FromString(Struct->GetName()),
				FOnVerifyTextChanged::CreateSP(this, &FHeaderViewStructListItem::OnVerifyRenameTextChanged, WeakStruct),
				FOnTextCommitted::CreateSP(this, &FHeaderViewStructListItem::OnRenameTextComitted, WeakStruct)
 			);
		}
	}
}

FHeaderViewStructListItem::FHeaderViewStructListItem(TWeakObjectPtr<UUserDefinedStruct> InStruct)
{
	if (UUserDefinedStruct* Struct = InStruct.Get())
	{
		// Avoid lots of reallocations
		RawItemString.Reserve(512);
		RichTextString.Reserve(512);

		// Format comment
		{
			FString Comment = FStructureEditorUtils::GetTooltip(Struct);
			if (Comment.IsEmpty())
			{
				Comment = TEXT("Please add a struct description");
			} 
			FormatCommentString(Comment, RawItemString, RichTextString);
		}

		// Add the USTRUCT specifiers
		// i.e. USTRUCT(BlueprintType)
		{
			// Always add BlueprintType
			RawItemString += TEXT("\nUSTRUCT(BlueprintType)");
			RichTextString += FString::Printf(TEXT("\n<%s>USTRUCT</>(BlueprintType)"), *HeaderViewSyntaxDecorators::MacroDecorator);
		}

		// Add the class declaration line
		// i.e. struct StructName
		{
			const FString StructName = Struct->GetPrefixCPP() + Struct->GetName();

			bIsValidName = IsValidCPPIdentifier(StructName);
			const FString& NameDecorator = bIsValidName ? HeaderViewSyntaxDecorators::TypenameDecorator : HeaderViewSyntaxDecorators::ErrorDecorator;

			RawItemString += FString::Printf(TEXT("\nstruct %s\n{\n\tGENERATED_BODY()"), *StructName);
			RichTextString += FString::Printf(TEXT("\n<%s>struct</> <%s>%s</>\n{\n\t<%s>GENERATED_BODY</>()"), 
				*HeaderViewSyntaxDecorators::KeywordDecorator,
				*NameDecorator, 
				*StructName,
				*HeaderViewSyntaxDecorators::MacroDecorator
			);
		}

		// normalize to platform newlines
		UE::String::ToHostLineEndingsInline(RawItemString);
		UE::String::ToHostLineEndingsInline(RichTextString);
	}
}

FString FHeaderViewStructListItem::GetRenamedStructPath(const UUserDefinedStruct* Struct, const FString& NewName) const
{
	check(Struct);
	FString NewPath = Struct->GetPathName();
	int32 Index = 0;
	if (!ensure(NewPath.FindLastChar(TEXT('/'), Index)))
	{
		// AssetViewUtils::IsValidObjectPathForCreate will return false for an empty string
		return TEXT("");
	}
	NewPath.LeftInline(Index);
	NewPath /= FString::Printf(TEXT("%s.%s"), *NewName, *NewName);
	return NewPath;
}

bool FHeaderViewStructListItem::OnVerifyRenameTextChanged(const FText& InNewName, FText& OutErrorText, TWeakObjectPtr<UUserDefinedStruct> InStruct)
{
	if (const UUserDefinedStruct* Struct = InStruct.Get())
	{
		const FString NewPath = GetRenamedStructPath(Struct, InNewName.ToString());
		if (!AssetViewUtils::IsValidObjectPathForCreate(NewPath, Struct->GetClass(), OutErrorText))
		{
			return false;
		}

		if (!IsValidCPPIdentifier(InNewName.ToString()))
		{
			OutErrorText = InvalidCPPIdentifierErrorText;
			return false;
		}

		return true;
	}

	return false;
}

void FHeaderViewStructListItem::OnRenameTextComitted(const FText& CommittedText, ETextCommit::Type TextCommitType, TWeakObjectPtr<UUserDefinedStruct> InStruct)
{
	if (TextCommitType == ETextCommit::OnEnter)
	{
		FText ErrorText;
		if (OnVerifyRenameTextChanged(CommittedText, ErrorText, InStruct))
		{
			if (UUserDefinedStruct* Struct = InStruct.Get())
			{
				const FString NewPath = GetRenamedStructPath(Struct, CommittedText.ToString());
				FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");
				TArray<FAssetRenameData> AssetToRename = { FAssetRenameData(Struct, NewPath) };
				AssetToolsModule.Get().RenameAssets(AssetToRename);
				FStructureEditorUtils::OnStructureChanged(Struct, FStructureEditorUtils::Unknown);
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
