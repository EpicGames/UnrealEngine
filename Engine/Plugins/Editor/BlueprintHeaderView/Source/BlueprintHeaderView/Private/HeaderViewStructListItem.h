// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SBlueprintHeaderView.h"

namespace ETextCommit { enum Type : int; }

class FMenuBuilder;

/** A header view list item that displays the struct declaration */
struct FHeaderViewStructListItem : public FHeaderViewListItem
{
public:
	/** Creates a list item for the Header view representing a class declaration for the given blueprint */
	static FHeaderViewListItemPtr Create(TWeakObjectPtr<UUserDefinedStruct> InStruct);

	//~ FHeaderViewListItem Interface
	virtual void ExtendContextMenu(FMenuBuilder& InMenuBuilder, TWeakObjectPtr<UObject> InAsset) override;
	//~ End FHeaderViewListItem Interface
protected:
	FHeaderViewStructListItem(TWeakObjectPtr<UUserDefinedStruct> InStruct);

	FString GetRenamedStructPath(const UUserDefinedStruct* Struct, const FString& NewName) const;

	bool OnVerifyRenameTextChanged(const FText& InNewName, FText& OutErrorText, TWeakObjectPtr<UUserDefinedStruct> InStruct);

	void OnRenameTextComitted(const FText& CommittedText, ETextCommit::Type TextCommitType, TWeakObjectPtr<UUserDefinedStruct> InStruct);

protected:
	/** Whether this class name is valid C++ (no spaces, special chars, etc) */
	bool bIsValidName = true;
};
