// Copyright Epic Games, Inc. All Rights Reserved.

#include "HeaderViewVariableListItem.h"
#include "EdMode.h"
#include "Kismet2/Kismet2NameValidators.h"
#include "StructUtils/UserDefinedStruct.h"
#include "String/LineEndings.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/StructureEditorUtils.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

#define LOCTEXT_NAMESPACE "FHeaderViewVariableListItem"

FHeaderViewListItemPtr FHeaderViewVariableListItem::Create(const FBPVariableDescription* VariableDesc, const FProperty& VarProperty)
{
	return MakeShareable(new FHeaderViewVariableListItem(VariableDesc, VarProperty));
}

void FHeaderViewVariableListItem::ExtendContextMenu(FMenuBuilder& InMenuBuilder, TWeakObjectPtr<UObject> InAsset)
{
	if (!IllegalName.IsNone())
	{
		InMenuBuilder.AddVerifiedEditableText(LOCTEXT("RenameItem", "Rename Variable"),
			LOCTEXT("RenameItemTooltip", "Renames this variable in the Blueprint\nThis variable name is not a legal C++ identifier."),
			FSlateIcon(),
			FText::FromName(IllegalName),
			FOnVerifyTextChanged::CreateSP(this, &FHeaderViewVariableListItem::OnVerifyRenameTextChanged, InAsset),
			FOnTextCommitted::CreateSP(this, &FHeaderViewVariableListItem::OnRenameTextCommitted, InAsset)
		);
	}
}

FHeaderViewVariableListItem::FHeaderViewVariableListItem(const FBPVariableDescription* VariableDesc, const FProperty& VarProperty)
{
	// Format comment
	{
		FString Comment = VarProperty.GetMetaData(FBlueprintMetadata::MD_Tooltip);
		if (Comment.IsEmpty())
		{
			Comment = TEXT("Please add a variable description");
		}

		FormatCommentString(Comment, RawItemString, RichTextString);
	}

	// Add a static_assert if the user needs to tell these variables how to replicate
	// i.e. static_assert(false, "You will need to add DOREPLIFETIME(ClassName, VarName) to GetLifeTimeReplicatedProps");
	if (VarProperty.HasAnyPropertyFlags(CPF_Net))
	{
		FString ClassName = GetOwningClassName(VarProperty);
		if (!VariableDesc || VariableDesc->ReplicationCondition == COND_None)
		{
			RawItemString += FString::Printf(TEXT("\nstatic_assert(false, \"You will need to add DOREPLIFETIME(%s, %s) to GetLifetimeReplicatedProps\");"), *ClassName, *VarProperty.GetAuthoredName());
			RichTextString += FString::Printf(TEXT("\n<%s>static_assert</>(<%s>false</>, \"You will need to add DOREPLIFETIME(%s, %s) to GetLifetimeReplicatedProps\");"), 
				*HeaderViewSyntaxDecorators::KeywordDecorator, 
				*HeaderViewSyntaxDecorators::KeywordDecorator, 
				*ClassName, 
				*VarProperty.GetAuthoredName());
		}
		else
		{
			FString ConditionString = StaticEnum<ELifetimeCondition>()->GetAuthoredNameStringByValue(VariableDesc->ReplicationCondition);
			RawItemString += FString::Printf(TEXT("\nstatic_assert(false, \"You will need to add DOREPLIFETIME_WITH_PARAMS(%s, %s, %s) to GetLifetimeReplicatedProps\");"), *ClassName, *VarProperty.GetAuthoredName(), *ConditionString);
			RichTextString += FString::Printf(TEXT("\n<%s>static_assert</>(<%s>false</>, \"You will need to add DOREPLIFETIME_WITH_PARAMS(%s, %s, %s) to GetLifetimeReplicatedProps\");"),
				*HeaderViewSyntaxDecorators::KeywordDecorator,
				*HeaderViewSyntaxDecorators::KeywordDecorator,
				*ClassName,
				*VarProperty.GetAuthoredName(),
				*ConditionString);
		}
	}

	// Add Delegate type declaration if needed
	// i.e. DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FDelegateTypeName, ParamType, ParamName);
	if (const FMulticastDelegateProperty* AsDelegate = CastField<FMulticastDelegateProperty>(&VarProperty))
	{
		FormatDelegateDeclaration(*AsDelegate);
	}

	// Add Deprecation message if present
	if (VarProperty.HasAnyPropertyFlags(CPF_Deprecated))
	{
		FString DeprecationMessage = VarProperty.GetMetaData(FBlueprintMetadata::MD_DeprecationMessage);
		if (DeprecationMessage.IsEmpty())
		{
			DeprecationMessage = TEXT("Please add a deprecation message.");
		}

		const FString EngineVersionString = FEngineVersion::Current().ToString(EVersionComponent::Patch);

		RawItemString += FString::Printf(TEXT("\nUE_DEPRECATED(%s, \"%s\")"), *EngineVersionString, *DeprecationMessage);
		RichTextString += FString::Printf(TEXT("\n<%s>UE_DEPRECATED</>(%s, \"%s\")"), *HeaderViewSyntaxDecorators::MacroDecorator, *EngineVersionString, *DeprecationMessage);
	}

	// Add the UPROPERTY specifiers
	// i.e. UPROPERTY(BlueprintReadWrite, Category="Variable Category")
	{
		const FString AdditionalSpecifiers = GetConditionalUPropertySpecifiers(VarProperty);

		// Always add Blueprintable and BlueprintType
		RawItemString += FString::Printf(TEXT("\nUPROPERTY(%s)"), *AdditionalSpecifiers);
		RichTextString += FString::Printf(TEXT("\n<%s>UPROPERTY</>(%s)"), *HeaderViewSyntaxDecorators::MacroDecorator, *AdditionalSpecifiers);
	}

	// Add the variable declaration line
	// i.e. Type VariableName;
	{
		const FString Typename = GetCPPTypenameForProperty(&VarProperty, /*bIsMemberProperty=*/true);
		const FString VarName = VarProperty.GetAuthoredName();
		const bool bLegalName = IsValidCPPIdentifier(VarName);

		const FString* IdentifierDecorator = &HeaderViewSyntaxDecorators::IdentifierDecorator;
		if (!bLegalName)
		{
			IllegalName = FName(VarName);
			IdentifierDecorator = &HeaderViewSyntaxDecorators::ErrorDecorator;
		}


		RawItemString += FString::Printf(TEXT("\n%s %s;"), *Typename, *VarName);
		RichTextString += FString::Printf(TEXT("\n<%s>%s</> <%s>%s</>;"), 
			*HeaderViewSyntaxDecorators::TypenameDecorator,
			*Typename.Replace(TEXT("<"), TEXT("&lt;")).Replace(TEXT(">"), TEXT("&gt;")),
			**IdentifierDecorator,
			*VarName);
	}

	// indent item
	RawItemString.InsertAt(0, TEXT("\t"));
	RichTextString.InsertAt(0, TEXT("\t"));
	RawItemString.ReplaceInline(TEXT("\n"), TEXT("\n\t"));
	RichTextString.ReplaceInline(TEXT("\n"), TEXT("\n\t"));

	// normalize to platform newlines
	UE::String::ToHostLineEndingsInline(RawItemString);
	UE::String::ToHostLineEndingsInline(RichTextString);
}

void FHeaderViewVariableListItem::FormatDelegateDeclaration(const FMulticastDelegateProperty& DelegateProp)
{
	if (const UFunction* SigFunction = DelegateProp.SignatureFunction)
	{
		static const TMap<int32, FString> ParamStringsMap =
		{
			{-1, TEXT("_TooManyParams")},
			{0, TEXT("")},
			{1, TEXT("_OneParam")},
			{2, TEXT("_TwoParams")},
			{3, TEXT("_ThreeParams")},
			{4, TEXT("_FourParams")},
			{5, TEXT("_FiveParams")},
			{6, TEXT("_SixParams")},
			{7, TEXT("_SevenParams")},
			{8, TEXT("_EightParams")},
			{9, TEXT("_NineParams")}
		};

		const FString* NumParamsString = ParamStringsMap.Find(SigFunction->NumParms);
		if (!NumParamsString)
		{
			NumParamsString = &ParamStringsMap[-1];
		}

		TArray<FString> MacroParams;
		MacroParams.Emplace(GetCPPTypenameForProperty(&DelegateProp));

		for (TFieldIterator<FProperty> ParamIt(SigFunction); ParamIt; ++ParamIt)
		{
			MacroParams.Emplace(GetCPPTypenameForProperty(*ParamIt));
			if (ParamIt->HasAnyPropertyFlags(CPF_ReferenceParm))
			{
				MacroParams.Last().Append(TEXT("&"));
			}

			MacroParams.Emplace(ParamIt->GetAuthoredName());
		}

		RawItemString += FString::Printf(TEXT("\nDECLARE_DYNAMIC_MULTICAST_DELEGATE%s(%s);"), **NumParamsString, *FString::Join(MacroParams, TEXT(", ")));

		// Add rich text decorators for macro parameters
		for (int32 ParmIdx = 0; ParmIdx < MacroParams.Num(); ++ParmIdx)
		{
			// the delegate name (idx 0) and parameter types (odd number idx) are typenames, all other parameters are identifiers
			const bool bIsTypename = (ParmIdx == 0 || ParmIdx % 2 != 0);
			MacroParams[ParmIdx] = FString::Printf(TEXT("<%s>%s</>"), bIsTypename ? *HeaderViewSyntaxDecorators::TypenameDecorator : *HeaderViewSyntaxDecorators::IdentifierDecorator, *MacroParams[ParmIdx]);
		}

		RichTextString += FString::Printf(TEXT("\n<%s>DECLARE_DYNAMIC_MULTICAST_DELEGATE%s</>(%s);"), *HeaderViewSyntaxDecorators::MacroDecorator, **NumParamsString, *FString::Join(MacroParams, TEXT(", ")));
	}
}

FString FHeaderViewVariableListItem::GetConditionalUPropertySpecifiers(const FProperty& VarProperty) const
{
	TArray<FString> PropertySpecifiers;

	if (!VarProperty.HasMetaData(FBlueprintMetadata::MD_Private) || !VarProperty.GetBoolMetaData(FBlueprintMetadata::MD_Private))
	{
		if (VarProperty.HasAnyPropertyFlags(CPF_BlueprintAssignable))
		{
			PropertySpecifiers.Emplace(TEXT("BlueprintAssignable"));
		}
		else if (VarProperty.HasAnyPropertyFlags(CPF_Edit))
		{
			PropertySpecifiers.Emplace(TEXT("BlueprintReadWrite"));
		}
		else
		{
			PropertySpecifiers.Emplace(TEXT("BlueprintReadOnly"));
		}
	}

	if (!VarProperty.HasAnyPropertyFlags(CPF_Edit))
	{
		PropertySpecifiers.Emplace(TEXT("VisibleAnywhere"));
	}
	else if (!VarProperty.HasAnyPropertyFlags(CPF_DisableEditOnInstance | CPF_DisableEditOnTemplate))
	{
		PropertySpecifiers.Emplace(TEXT("EditAnywhere"));
	}
	else if (VarProperty.HasAnyPropertyFlags(CPF_DisableEditOnInstance))
	{
		PropertySpecifiers.Emplace(TEXT("EditDefaultsOnly"));
	}
	else
	{
		PropertySpecifiers.Emplace(TEXT("EditInstanceOnly"));
	}

	if (VarProperty.HasMetaData(FBlueprintMetadata::MD_FunctionCategory))
	{
		PropertySpecifiers.Emplace(FString::Printf(TEXT("Category=\"%s\""), *VarProperty.GetMetaData(FBlueprintMetadata::MD_FunctionCategory)));
	}

	if (VarProperty.HasAnyPropertyFlags(CPF_Net))
	{
		if (VarProperty.HasAnyPropertyFlags(CPF_RepNotify))
		{
			PropertySpecifiers.Emplace(FString::Printf(TEXT("ReplicatedUsing=\"OnRep_%s\""), *VarProperty.GetAuthoredName()));
		}
		else
		{
			PropertySpecifiers.Emplace(TEXT("Replicated"));
		}
	}

	if (VarProperty.HasAnyPropertyFlags(CPF_Interp))
	{
		PropertySpecifiers.Emplace(TEXT("Interp"));
	}

	if (VarProperty.HasAnyPropertyFlags(CPF_Config))
	{
		PropertySpecifiers.Emplace(TEXT("Config"));
	}

	if (VarProperty.HasAnyPropertyFlags(CPF_Transient))
	{
		PropertySpecifiers.Emplace(TEXT("Transient"));
	}

	if (VarProperty.HasAnyPropertyFlags(CPF_SaveGame))
	{
		PropertySpecifiers.Emplace(TEXT("SaveGame"));
	}

	if (VarProperty.HasAnyPropertyFlags(CPF_AdvancedDisplay))
	{
		PropertySpecifiers.Emplace(TEXT("AdvancedDisplay"));
	}

	if (VarProperty.HasMetaData(FEdMode::MD_MakeEditWidget) && VarProperty.GetBoolMetaData(FEdMode::MD_MakeEditWidget))
	{
		PropertySpecifiers.Emplace(TEXT("MakeEditWidget"));
	}

	// meta specifiers
	{
		if (const TMap<FName, FString>* MetaDataMap = VarProperty.GetMetaDataMap())
		{
			// These metadata keys are handled explicitly elsewhere
			static const TArray<FName> IgnoreMetaData = 
			{
				FBlueprintMetadata::MD_DeprecationMessage,
				FBlueprintMetadata::MD_FunctionCategory,
				FBlueprintMetadata::MD_Tooltip,
				FBlueprintMetadata::MD_Private,
				FEdMode::MD_MakeEditWidget
			};

			TArray<FString> MetaSpecifiers;

			for (const TPair<FName, FString>& MetaPair : *MetaDataMap)
			{
				// Don't take this metadata if we handled it explicitly
				if (IgnoreMetaData.Contains(MetaPair.Key))
				{
					continue;
				}

				// Don't add the DisplayName metadata key if the display name is just the friendly name generated by the editor anyway
				if (MetaPair.Key == FBlueprintMetadata::MD_DisplayName)
				{
					if (MetaPair.Value == FName::NameToDisplayString(VarProperty.GetName(), !!CastField<FBoolProperty>(&VarProperty)))
					{
						continue;
					}
				}

				MetaSpecifiers.Emplace(FString::Printf(TEXT("%s=\"%s\""), *MetaPair.Key.ToString(), *MetaPair.Value));
			}

			if (!MetaSpecifiers.IsEmpty())
			{
				PropertySpecifiers.Emplace(FString::Printf(TEXT("meta=(%s)"), *FString::Join(MetaSpecifiers, TEXT(", "))));
			}
		}
	}

	return FString::Join(PropertySpecifiers, TEXT(", "));
}

FString FHeaderViewVariableListItem::GetOwningClassName(const FProperty& VarProperty) const
{
	UClass* OwningClass = VarProperty.GetOwnerClass();
	if (OwningClass && OwningClass->ClassGeneratedBy)
	{
		return OwningClass->GetPrefixCPP() + OwningClass->ClassGeneratedBy->GetName();
	}

	return TEXT("");
}

bool FHeaderViewVariableListItem::OnVerifyRenameTextChanged(const FText& InNewName, FText& OutErrorText, TWeakObjectPtr<UObject> WeakAsset)
{
	if (!IsValidCPPIdentifier(InNewName.ToString()))
	{
		OutErrorText = InvalidCPPIdentifierErrorText;
		return false;
	}

	UObject* Asset = WeakAsset.Get();

	if (const UBlueprint* Blueprint = Cast<UBlueprint>(Asset))
	{
		FKismetNameValidator NameValidator(Blueprint, NAME_None, nullptr);
		const EValidatorResult Result = NameValidator.IsValid(InNewName.ToString());
		if (Result != EValidatorResult::Ok)
		{
			OutErrorText = GetErrorTextFromValidatorResult(Result);
			return false;
		}

		return true;
	}

	if (const UUserDefinedStruct* Struct = Cast<UUserDefinedStruct>(Asset))
	{
		return FStructureEditorUtils::IsUniqueVariableFriendlyName(Struct, InNewName.ToString());
	}

	return false;
}

void FHeaderViewVariableListItem::OnRenameTextCommitted(const FText& CommittedText, ETextCommit::Type TextCommitType, TWeakObjectPtr<UObject> WeakAsset)
{
	if (TextCommitType == ETextCommit::OnEnter)
	{
		if (UObject* Asset = WeakAsset.Get())
		{
			const FString& CommittedString = CommittedText.ToString();
			if (IsValidCPPIdentifier(CommittedString))
			{
				if (UBlueprint* Blueprint = Cast<UBlueprint>(Asset))
				{
					FBlueprintEditorUtils::RenameMemberVariable(Blueprint, IllegalName, FName(CommittedString));
				}
				else if (UUserDefinedStruct* Struct = Cast<UUserDefinedStruct>(Asset))
				{
					FStructureEditorUtils::RenameVariable(Struct, IllegalName.ToString(), CommittedString);
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
