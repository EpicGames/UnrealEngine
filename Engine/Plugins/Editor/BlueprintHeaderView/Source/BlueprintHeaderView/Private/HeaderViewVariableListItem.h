// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SBlueprintHeaderView.h"

namespace ETextCommit { enum Type : int; }

struct FBPVariableDescription;

/** A header view list item that displays a variable declaration */
struct FHeaderViewVariableListItem : public FHeaderViewListItem
{
	/** Creates a list item for the Header view representing a variable declaration for the given blueprint variable */
	static FHeaderViewListItemPtr Create(const FBPVariableDescription* VariableDesc, const FProperty& VarProperty);

	//~ FHeaderViewListItem Interface
	virtual void ExtendContextMenu(FMenuBuilder& InMenuBuilder, TWeakObjectPtr<UObject> InAsset) override;
	//~ End FHeaderViewListItem Interface

protected:
	FHeaderViewVariableListItem(const FBPVariableDescription* VariableDesc, const FProperty& VarProperty);

	/** Formats a line declaring a delegate type and appends it to the item strings */
	void FormatDelegateDeclaration(const FMulticastDelegateProperty& DelegateProp);

	/** Returns a string containing the specifiers for the UPROPERTY line */
	FString GetConditionalUPropertySpecifiers(const FProperty& VarProperty) const;

	/** Returns the name of the owning class */
	FString GetOwningClassName(const FProperty& VarProperty) const;

	bool OnVerifyRenameTextChanged(const FText& InNewName, FText& OutErrorText, TWeakObjectPtr<UObject> WeakAsset);
	void OnRenameTextCommitted(const FText& CommittedText, ETextCommit::Type TextCommitType, TWeakObjectPtr<UObject> WeakAsset);
protected:

	/** None if the name is legal, else holds the name of the variable */
	FName IllegalName = NAME_None;
};
