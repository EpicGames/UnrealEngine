// Copyright Epic Games, Inc. All Rights Reserved.

#include "SBlueprintHeaderView.h"
#include "BlueprintHeaderView.h"
#include "DetailsViewArgs.h"
#include "Kismet2/Kismet2NameValidators.h"
#include "Widgets/Layout/SSpacer.h"
//#include "Widgets/Layout/SScrollBox.h"
#include "EdGraph/EdGraph.h"
#include "Widgets/Input/SComboButton.h"
#include "Framework/Commands/UICommandList.h"
#include "Widgets/Text/SRichTextBlock.h"
#include "Framework/Views/TableViewMetadata.h"
#include "Widgets/Input/SButton.h"
#include "IDetailsView.h"
#include "PropertyEditorModule.h"
#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"
#include "StructUtils/UserDefinedStruct.h"
#include "Framework/Text/SlateTextRun.h"
#include "Framework/Commands/GenericCommands.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HeaderViewClassListItem.h"
#include "HeaderViewStructListItem.h"
#include "HeaderViewFunctionListItem.h"
#include "HeaderViewVariableListItem.h"
#include "EdGraphSchema_K2.h"
#include "K2Node_FunctionEntry.h"
#include "String/LineEndings.h"
#include "HAL/PlatformApplicationMisc.h"
#include "BlueprintHeaderViewSettings.h"
#include "Internationalization/Regex.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Editor/EditorEngine.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "SBlueprintHeaderView"

DEFINE_LOG_CATEGORY(LogBlueprintHeaderView)

extern UNREALED_API UEditorEngine* GEditor;

// HeaderViewSyntaxDecorators /////////////////////////////////////////////////

namespace HeaderViewSyntaxDecorators
{
	const FString CommentDecorator = TEXT("comment");
	const FString ErrorDecorator = TEXT("error");
	const FString IdentifierDecorator = TEXT("identifier");
	const FString KeywordDecorator = TEXT("keyword");
	const FString MacroDecorator = TEXT("macro");
	const FString TypenameDecorator = TEXT("typename");
}

// FHeaderViewSyntaxDecorator /////////////////////////////////////////////////

namespace
{
	class FHeaderViewSyntaxDecorator : public ITextDecorator
	{
	public:

		static TSharedRef<FHeaderViewSyntaxDecorator> Create(FString InName, const FSlateColor& InColor)
		{
			return MakeShareable(new FHeaderViewSyntaxDecorator(MoveTemp(InName), InColor));
		}

		bool Supports(const FTextRunParseResults& RunInfo, const FString& Text) const override
		{
			return RunInfo.Name == DecoratorName;
		}

		TSharedRef<ISlateRun> Create(const TSharedRef<class FTextLayout>& TextLayout, const FTextRunParseResults& RunParseResult, const FString& OriginalText, const TSharedRef<FString>& ModelText, const ISlateStyle* Style) override
		{
			FRunInfo RunInfo(RunParseResult.Name);
			for (const TPair<FString, FTextRange>& Pair : RunParseResult.MetaData)
			{
				RunInfo.MetaData.Add(Pair.Key, OriginalText.Mid(Pair.Value.BeginIndex, Pair.Value.EndIndex - Pair.Value.BeginIndex));
			}

			ModelText->Append(OriginalText.Mid(RunParseResult.ContentRange.BeginIndex, RunParseResult.ContentRange.Len()));

			FTextBlockStyle TextStyle = FBlueprintHeaderViewModule::HeaderViewTextStyle;
			TextStyle.SetColorAndOpacity(TextColor);

			return FSlateTextRun::Create(RunInfo, ModelText, TextStyle);
		}

	private:

		FHeaderViewSyntaxDecorator(FString&& InName, const FSlateColor& InColor)
			: DecoratorName(MoveTemp(InName))
			, TextColor(InColor)
		{
		}

	private:

		/** Name of this decorator */
		FString DecoratorName;

		/** Color of this decorator */
		FSlateColor TextColor;

	};

}

// FHeaderViewListItem ////////////////////////////////////////////////////////

const FText FHeaderViewListItem::InvalidCPPIdentifierErrorText = LOCTEXT("CPPIdentifierError", "Name is not a valid C++ Identifier");

TSharedRef<SWidget> FHeaderViewListItem::GenerateWidgetForItem()
{
	const UBlueprintHeaderViewSettings* HeaderViewSettings = GetDefault<UBlueprintHeaderViewSettings>();
	const FHeaderViewSyntaxColors& SyntaxColors = HeaderViewSettings->SyntaxColors;

	return SNew(SBox)
		.HAlign(HAlign_Fill)
		.Padding(FMargin(4.0f))
		[
			SNew(SRichTextBlock)
			.Text(FText::FromString(RichTextString))
			.TextStyle(&FBlueprintHeaderViewModule::HeaderViewTextStyle)
			+SRichTextBlock::Decorator(FHeaderViewSyntaxDecorator::Create(HeaderViewSyntaxDecorators::CommentDecorator, SyntaxColors.Comment))
			+SRichTextBlock::Decorator(FHeaderViewSyntaxDecorator::Create(HeaderViewSyntaxDecorators::ErrorDecorator, SyntaxColors.Error))
			+SRichTextBlock::Decorator(FHeaderViewSyntaxDecorator::Create(HeaderViewSyntaxDecorators::IdentifierDecorator, SyntaxColors.Identifier))
			+SRichTextBlock::Decorator(FHeaderViewSyntaxDecorator::Create(HeaderViewSyntaxDecorators::KeywordDecorator, SyntaxColors.Keyword))
			+SRichTextBlock::Decorator(FHeaderViewSyntaxDecorator::Create(HeaderViewSyntaxDecorators::MacroDecorator, SyntaxColors.Macro))
			+SRichTextBlock::Decorator(FHeaderViewSyntaxDecorator::Create(HeaderViewSyntaxDecorators::TypenameDecorator, SyntaxColors.Typename))
		];
}

TSharedPtr<FHeaderViewListItem> FHeaderViewListItem::Create(FString InRawString, FString InRichText)
{
	return MakeShareable(new FHeaderViewListItem(MoveTemp(InRawString), MoveTemp(InRichText)));
}

FHeaderViewListItem::FHeaderViewListItem(FString&& InRawString, FString&& InRichText)
	: RichTextString(MoveTemp(InRichText))
	, RawItemString(MoveTemp(InRawString))
{
}

void FHeaderViewListItem::FormatCommentString(FString InComment, FString& OutRawString, FString& OutRichString)
{
	// normalize newlines to \n
	UE::String::FromHostLineEndingsInline(InComment);

	if (InComment.Contains("\n"))
	{
		/**
		 * Format into a multiline C++ comment, like this one
		 */
		InComment = TEXT("/**\n") + InComment;
		InComment.ReplaceInline(TEXT("\n"), TEXT("\n * "));
		InComment.Append(TEXT("\n */"));
	}
	else
	{
		/** Format into a single line C++ comment, like this one */
		InComment = FString::Printf(TEXT("/** %s */"), *InComment);
	}

	// add the comment to the raw string representation
	OutRawString = InComment;

	// mark each line of the comment as the beginning and end of a comment style for the rich text representation
	InComment.ReplaceInline(TEXT("\n"), *FString::Printf(TEXT("</>\n<%s>"), *HeaderViewSyntaxDecorators::CommentDecorator));
	OutRichString = FString::Printf(TEXT("<%s>%s</>"), *HeaderViewSyntaxDecorators::CommentDecorator, *InComment);
}

FString FHeaderViewListItem::GetCPPTypenameForProperty(const FProperty* InProperty, bool bIsMemberProperty/*=false*/)
{
	if (InProperty)
	{
		FString ExtendedTypeText;
		FString Typename = InProperty->GetCPPType(&ExtendedTypeText) + ExtendedTypeText;
		
		if (bIsMemberProperty)
		{
			if (const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(InProperty))
			{
				// Replace native pointer with TObjectPtr
				Typename.LeftChopInline(1);
				Typename = FString::Printf(TEXT("TObjectPtr<%s>"), *Typename);
			}
		}

		return Typename;
	}
	else
	{
		return TEXT("");
	}
}

bool FHeaderViewListItem::IsValidCPPIdentifier(const FString& InIdentifier)
{
	// Only matches strings that start with a Letter or Underscore with any number of letters, digits or underscores following
	// _Legal, Legal0, legal_0
	// 0Illegal, Illegal&, Illegal-0
	static const FRegexPattern RegexPattern = FRegexPattern(TEXT("^[A-Za-z_]\\w*$"));
	return FRegexMatcher(RegexPattern, InIdentifier).FindNext();
}

const FText& FHeaderViewListItem::GetErrorTextFromValidatorResult(EValidatorResult Result)
{
	static const TMap<EValidatorResult, FText> ErrorTextMap =
	{
		{EValidatorResult::AlreadyInUse, LOCTEXT("AlreadyInUse", "The name is already in use.")},
		{EValidatorResult::EmptyName, LOCTEXT("EmptyName", "The name is empty.")},
		{EValidatorResult::TooLong, LOCTEXT("TooLong", "The name is too long.")},
		{EValidatorResult::LocallyInUse, LOCTEXT("LocallyInUse", "The name is already in use locally.")}
	};

	if (const FText* ErrorText = ErrorTextMap.Find(Result))
	{
		return *ErrorText;
	}

	return FText::GetEmpty();
}

// SBlueprintHeaderView ///////////////////////////////////////////////////////

void SBlueprintHeaderView::Construct(const FArguments& InArgs)
{
	const float PaddingAmount = 8.0f;
	SelectedBlueprint = nullptr;

	CommandList = MakeShared<FUICommandList>();
	
	CommandList->MapAction(FGenericCommands::Get().Copy,
		FExecuteAction::CreateRaw(this, &SBlueprintHeaderView::OnCopy),
		FCanExecuteAction::CreateRaw(this, &SBlueprintHeaderView::CanCopy));

	CommandList->MapAction(FGenericCommands::Get().SelectAll,
		FExecuteAction::CreateRaw(this, &SBlueprintHeaderView::OnSelectAll));

	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(FMargin(PaddingAmount))
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("ClassPickerLabel", "Displaying Blueprint:"))
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			[
				SNew(SSpacer)
				.Size(FVector2D(PaddingAmount))
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			[
				SNew(SBox)
				.WidthOverride(400.0f)
				[
					SAssignNew(ClassPickerComboButton, SComboButton)
					.OnGetMenuContent(this, &SBlueprintHeaderView::GetClassPickerMenuContent)
					.ButtonContent()
					[
						SNew(STextBlock)
						.Text(this, &SBlueprintHeaderView::GetClassPickerText)
					]
				]
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
				.ToolTipText(LOCTEXT("BrowseToBlueprintTooltip", "Browse to Selected Blueprint in Content Browser"))
				.OnClicked(this, &SBlueprintHeaderView::BrowseToAssetClicked)
				.ContentPadding(4.0f)
				.Content()
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Icons.BrowseContent"))
				]
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
				.ToolTipText(LOCTEXT("EditBlueprintTooltip", "Open Selected Blueprint in Blueprint Editor"))
				.OnClicked(this, &SBlueprintHeaderView::OpenAssetEditorClicked)
				.ContentPadding(4.0f)
				.Content()
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("Icons.Edit"))
				]
			]
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			[
				SNew(SComboButton)
				.HasDownArrow(false)
				.ForegroundColor(FSlateColor::UseForeground())
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.OnGetMenuContent(this, &SBlueprintHeaderView::GetSettingsMenuContent)
				.ButtonContent()
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Center)
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("Icons.Toolbar.Settings"))
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
					+SHorizontalBox::Slot()
					.Padding(FMargin(5, 0, 0, 0))
					.VAlign(VAlign_Center)
					.AutoWidth()
					[
						SNew(STextBlock)
						.TextStyle(FAppStyle::Get(), "NormalText")
						.Text(LOCTEXT("SettingsLabel", "Settings"))
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
				]
			]
		]
		+SVerticalBox::Slot()
		.Padding(FMargin(PaddingAmount))
		[
			//TODO: Add a scroll bar when this overflows from the available space
// 			SNew(SScrollBox)
// 			.Orientation(Orient_Horizontal)
// 			+SScrollBox::Slot()
// 			[
				SAssignNew(ListView, SListView<FHeaderViewListItemPtr>)
				.ListItemsSource(&ListItems)
				.OnGenerateRow(this, &SBlueprintHeaderView::GenerateRowForItem)
				.OnContextMenuOpening(this, &SBlueprintHeaderView::OnContextMenuOpening)
				.OnMouseButtonDoubleClick(this, &SBlueprintHeaderView::OnItemDoubleClicked)
//			]
		]
	];
}

SBlueprintHeaderView::~SBlueprintHeaderView()
{
	if (UBlueprint* Blueprint = SelectedBlueprint.Get())
	{
		Blueprint->OnChanged().RemoveAll(this);
	}
	else if (UUserDefinedStruct* Struct = SelectedStruct.Get())
	{
		Struct->OnStructChanged().RemoveAll(this);
	}
}

FReply SBlueprintHeaderView::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (CommandList.IsValid() && CommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void SBlueprintHeaderView::NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged)
{
	UBlueprintHeaderViewSettings* BlueprintHeaderViewSettings = GetMutableDefault<UBlueprintHeaderViewSettings>();
	check(BlueprintHeaderViewSettings);
	BlueprintHeaderViewSettings->SaveConfig();

	// repopulate the list view to update text style/sorting method based on settings
	RepopulateListView();
}

FText SBlueprintHeaderView::GetClassPickerText() const
{
	if (const UBlueprint* Blueprint = SelectedBlueprint.Get())
	{
		return FText::FromName(Blueprint->GetFName());
	}
	else if (const UUserDefinedStruct* Struct = SelectedStruct.Get())
	{
		return FText::FromName(Struct->GetFName());
	}

	return LOCTEXT("ClassPickerPickClass", "Select Blueprint Class");
}

TSharedRef<SWidget> SBlueprintHeaderView::GetClassPickerMenuContent()
{
	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>("ContentBrowser");

	FAssetPickerConfig AssetPickerConfig;
	AssetPickerConfig.SelectionMode = ESelectionMode::Single;
	AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateSP(this, &SBlueprintHeaderView::OnAssetSelected);
	AssetPickerConfig.Filter.ClassPaths.Add(UBlueprint::StaticClass()->GetClassPathName());
	AssetPickerConfig.Filter.ClassPaths.Add(UUserDefinedStruct::StaticClass()->GetClassPathName());
	AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;
	AssetPickerConfig.SaveSettingsName = TEXT("HeaderPreviewClassPickerSettings");
	TSharedRef<SWidget> AssetPickerWidget = ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig);

	return SNew(SBox)
		.HeightOverride(500.f)
		[
			AssetPickerWidget
		];
}

TSharedRef<SWidget> SBlueprintHeaderView::GetSettingsMenuContent()
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");	
	FDetailsViewArgs DetailsViewArgs;
	{
		DetailsViewArgs.bShowPropertyMatrixButton = false;
		DetailsViewArgs.bShowOptions = false;
		DetailsViewArgs.NotifyHook = this;
	}
	TSharedRef<IDetailsView> DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	DetailsView->SetObject(GetMutableDefault<UBlueprintHeaderViewSettings>());
	return DetailsView;
}

void SBlueprintHeaderView::OnAssetSelected(const FAssetData& SelectedAsset)
{
	ClassPickerComboButton->SetIsOpen(false);

	if (UBlueprint* OldBlueprint = SelectedBlueprint.Get())
	{
		OldBlueprint->OnChanged().RemoveAll(this);
	}
	else if (UUserDefinedStruct* OldStruct = SelectedStruct.Get())
	{
		OldStruct->OnStructChanged().RemoveAll(this);
	}

	SelectedBlueprint = nullptr;
	SelectedStruct = nullptr;

	if (UBlueprint* Blueprint = Cast<UBlueprint>(SelectedAsset.GetAsset()))
	{
		Blueprint->OnChanged().AddRaw(this, &SBlueprintHeaderView::OnBlueprintChanged);
		SelectedBlueprint = Blueprint;
	}
	else if (UUserDefinedStruct* UDS = Cast<UUserDefinedStruct>(SelectedAsset.GetAsset()))
	{
		UDS->OnStructChanged().AddRaw(this, &SBlueprintHeaderView::OnStructChanged);
		SelectedStruct = UDS;
	}

	RepopulateListView();
}

TSharedRef<ITableRow> SBlueprintHeaderView::GenerateRowForItem(FHeaderViewListItemPtr Item, const TSharedRef<STableViewBase>& OwnerTable) const
{
	return SNew(STableRow<FHeaderViewListItemPtr>, OwnerTable)
		.Style(&FBlueprintHeaderViewModule::HeaderViewTableRowStyle)
		.Content()
		[
			Item->GenerateWidgetForItem()
		];
}

void SBlueprintHeaderView::RepopulateListView()
{
	ListItems.Empty();

	if (UBlueprint* Blueprint = SelectedBlueprint.Get())
	{
		// Add the class declaration
		ListItems.Add(FHeaderViewClassListItem::Create(SelectedBlueprint));

		PopulateFunctionItems(Blueprint);

		PopulateVariableItems(Blueprint->SkeletonGeneratedClass);

		// Add the closing brace of the class
		ListItems.Add(FHeaderViewListItem::Create(TEXT("};"), TEXT("};")));
	}
	else if (UUserDefinedStruct* Struct = SelectedStruct.Get())
	{
		// Add the struct declaration
		ListItems.Add(FHeaderViewStructListItem::Create(Struct));

		PopulateVariableItems(Struct);

		// Add the closing brace of the class
		ListItems.Add(FHeaderViewListItem::Create(TEXT("};"), TEXT("};")));
	}

	ListView->RequestListRefresh();
}

void SBlueprintHeaderView::PopulateFunctionItems(const UBlueprint* Blueprint)
{
	if (Blueprint)
	{
		TArray<const UEdGraph*> FunctionGraphs;
		GatherFunctionGraphs(Blueprint, FunctionGraphs);

		// We should only add an access specifier line if the previous function was a different one
		int32 PrevAccessSpecifier = 0;
		for (const UEdGraph* FunctionGraph : FunctionGraphs)
		{
			if (FunctionGraph && !UEdGraphSchema_K2::IsConstructionScript(FunctionGraph))
			{
				TArray<UK2Node_FunctionEntry*> EntryNodes;
				FunctionGraph->GetNodesOfClass<UK2Node_FunctionEntry>(EntryNodes);

				if (ensure(EntryNodes.Num() == 1 && EntryNodes[0]))
				{
					int32 AccessSpecifier = EntryNodes[0]->GetFunctionFlags() & FUNC_AccessSpecifiers;
					AccessSpecifier = AccessSpecifier ? AccessSpecifier : FUNC_Public; //< Blueprint OnRep functions don't have one of these flags set, default to public in this case and others like it

					if (AccessSpecifier != PrevAccessSpecifier)
					{
						switch (AccessSpecifier)
						{
						default: 
						case FUNC_Public:
							ListItems.Add(FHeaderViewListItem::Create(TEXT("public:"), FString::Printf(TEXT("<%s>public</>:"), *HeaderViewSyntaxDecorators::KeywordDecorator)));
							break;
						case FUNC_Protected:
							ListItems.Add(FHeaderViewListItem::Create(TEXT("protected:"), FString::Printf(TEXT("<%s>protected</>:"), *HeaderViewSyntaxDecorators::KeywordDecorator)));
							break;
						case FUNC_Private:
							ListItems.Add(FHeaderViewListItem::Create(TEXT("private:"), FString::Printf(TEXT("<%s>private</>:"), *HeaderViewSyntaxDecorators::KeywordDecorator)));
							break;
						}
					}
					else
					{
						// add an empty line to space functions out
						ListItems.Add(FHeaderViewListItem::Create(TEXT(""), TEXT("")));
					}

					PrevAccessSpecifier = AccessSpecifier;

					ListItems.Add(FHeaderViewFunctionListItem::Create(EntryNodes[0]));
				}
			}
		}
	}
}

void SBlueprintHeaderView::GatherFunctionGraphs(const UBlueprint* Blueprint, TArray<const UEdGraph*>& OutFunctionGraphs)
{
	for (const UEdGraph* FunctionGraph : Blueprint->FunctionGraphs)
	{
		OutFunctionGraphs.Add(FunctionGraph);
	}

	const UBlueprintHeaderViewSettings* BlueprintHeaderViewSettings = GetDefault<UBlueprintHeaderViewSettings>();
	if (BlueprintHeaderViewSettings->SortMethod == EHeaderViewSortMethod::SortByAccessSpecifier)
	{
		OutFunctionGraphs.Sort([](const UEdGraph& LeftGraph, const UEdGraph& RightGraph)
			{
				TArray<UK2Node_FunctionEntry*> LeftEntryNodes;
				LeftGraph.GetNodesOfClass<UK2Node_FunctionEntry>(LeftEntryNodes);
				TArray<UK2Node_FunctionEntry*> RightEntryNodes;
				RightGraph.GetNodesOfClass<UK2Node_FunctionEntry>(RightEntryNodes);
				if (ensure(LeftEntryNodes.Num() == 1 && LeftEntryNodes[0] && RightEntryNodes.Num() == 1 && RightEntryNodes[0]))
				{
					const int32 LeftAccessSpecifier = LeftEntryNodes[0]->GetFunctionFlags() & FUNC_AccessSpecifiers;
					const int32 RightAccessSpecifier = RightEntryNodes[0]->GetFunctionFlags() & FUNC_AccessSpecifiers;

					return (LeftAccessSpecifier == FUNC_Public && RightAccessSpecifier != FUNC_Public) || (LeftAccessSpecifier == FUNC_Protected && RightAccessSpecifier == FUNC_Private);
				}

				return false;
			});
	}
}

void SBlueprintHeaderView::AddVariableItems(TArray<const FProperty*> VarProperties)
{
	const int32 Private = 2;
	const int32 Public = 1;

	const UBlueprintHeaderViewSettings* BlueprintHeaderViewSettings = GetDefault<UBlueprintHeaderViewSettings>();
	if (BlueprintHeaderViewSettings->SortMethod == EHeaderViewSortMethod::SortByAccessSpecifier)
	{
		VarProperties.Sort([](const FProperty& LeftProp, const FProperty& RightProp)
			{
				return !LeftProp.GetBoolMetaData(FBlueprintMetadata::MD_Private) && RightProp.GetBoolMetaData(FBlueprintMetadata::MD_Private);
			});
	}
	else if (BlueprintHeaderViewSettings->SortMethod == EHeaderViewSortMethod::SortForOptimalPadding)
	{
		SortPropertiesForPadding(VarProperties);
	}

	// We should only add an access specifier line if the previous variable was a different one
	int32 PrevAccessSpecifier = 0;
	for (const FProperty* VarProperty : VarProperties)
	{
		const int32 AccessSpecifier = VarProperty->GetBoolMetaData(FBlueprintMetadata::MD_Private) ? Private : Public;
		if (AccessSpecifier != PrevAccessSpecifier)
		{
			switch (AccessSpecifier)
			{
			case Public:
				ListItems.Add(FHeaderViewListItem::Create(TEXT("public:"), FString::Printf(TEXT("<%s>public</>:"), *HeaderViewSyntaxDecorators::KeywordDecorator)));
				break;
			case Private:
				ListItems.Add(FHeaderViewListItem::Create(TEXT("private:"), FString::Printf(TEXT("<%s>private</>:"), *HeaderViewSyntaxDecorators::KeywordDecorator)));
				break;
			}

			PrevAccessSpecifier = AccessSpecifier;
		}
		else
		{
			// add an empty line to space variables out
			ListItems.Add(FHeaderViewListItem::Create(TEXT(""), TEXT("")));
		}

		FBPVariableDescription* VariableDesc = nullptr;
		if (UBlueprint* Blueprint = SelectedBlueprint.Get())
		{
			VariableDesc = Blueprint->NewVariables.FindByPredicate([&VarProperty](const FBPVariableDescription& Desc)
				{
					return Desc.VarName == VarProperty->GetFName();
				});
		}

		ListItems.Add(FHeaderViewVariableListItem::Create(VariableDesc, *VarProperty));
	}
}

void SBlueprintHeaderView::PopulateVariableItems(const UStruct* Struct)
{
	TArray<const FProperty*> VarProperties;
	for (TFieldIterator<FProperty> PropertyIt(Struct, EFieldIteratorFlags::ExcludeSuper); PropertyIt; ++PropertyIt)
	{
		if (const FProperty* VarProperty = *PropertyIt)
		{
			if (VarProperty->HasAnyPropertyFlags(CPF_BlueprintVisible))
			{
				VarProperties.Add(VarProperty);
			}
		}
	}
	AddVariableItems(VarProperties);
}

void SBlueprintHeaderView::SortPropertiesForPadding(TArray<const FProperty*>& InOutProperties)
{
	TArray<const FProperty*> SortedProperties;
	SortedProperties.Reserve(InOutProperties.Num());

	auto GetNeededPadding = [] (int32 Offset, int32 Alignment)
	{
		if (Offset % Alignment == 0)
		{
			return 0;
		}

		return Alignment - (Offset % Alignment);
	};

	/**
	 * For optimal struct size, we always want to place a property that can be aligned
	 * on the current boundary with the least amount of padding. If there are multiple
	 * properties that could be placed with the same amount of padding, we should pick
	 * the one with the greatest alignment because that one will be harder to place
	 * perfectly.
	 * 
	 * Any types' size will be a multiple of its alignment, so an 8-aligned property
	 * can always be followed by a 4-aligned property with no padding, so if we're 
	 * at an 8-byte boundary we should always put the 8 first.
	 */

	int32 CurrentOffset = 0;
	while (!InOutProperties.IsEmpty())
	{
		int32 BestIndex = INDEX_NONE;
		int32 BestAlignment = 0;
		int32 BestPadding = 0;
		for (int32 PropIndex = 0; PropIndex < InOutProperties.Num(); ++PropIndex)
		{
			const FProperty* Property = InOutProperties[PropIndex];
			const int32 PropAlignment = Property->GetMinAlignment();
			const int32 PropPadding = GetNeededPadding(CurrentOffset, PropAlignment);

			if (BestIndex == INDEX_NONE
				|| PropPadding < BestPadding 
				|| (PropPadding == BestPadding && PropAlignment > BestAlignment))
			{
				BestIndex = PropIndex;
				BestAlignment = PropAlignment;
				BestPadding = PropPadding;
			}
		}

		SortedProperties.Add(InOutProperties[BestIndex]);
		InOutProperties.RemoveAt(BestIndex, EAllowShrinking::No);
		CurrentOffset += BestPadding + SortedProperties.Last()->GetSize();
	}

	InOutProperties = MoveTemp(SortedProperties);
}

TSharedPtr<SWidget> SBlueprintHeaderView::OnContextMenuOpening()
{
	FMenuBuilder MenuBuilder(true, CommandList);

	MenuBuilder.AddMenuEntry(FGenericCommands::Get().SelectAll);
	MenuBuilder.AddMenuEntry(FGenericCommands::Get().Copy);

	TArray<FHeaderViewListItemPtr> SelectedListItems;
	ListView->GetSelectedItems(SelectedListItems);
	if (SelectedListItems.Num() == 1)
	{
		TWeakObjectPtr<UObject> SelectedAsset = SelectedBlueprint.IsValid() ? (UObject*)SelectedBlueprint.Get() : (UObject*)SelectedStruct.Get();
		SelectedListItems[0]->ExtendContextMenu(MenuBuilder, SelectedAsset);
	}

	return MenuBuilder.MakeWidget();
}

void SBlueprintHeaderView::OnItemDoubleClicked(FHeaderViewListItemPtr Item)
{
	TWeakObjectPtr<UObject> SelectedAsset = SelectedBlueprint.IsValid() ? (UObject*)SelectedBlueprint.Get() : (UObject*)SelectedStruct.Get();
	Item->OnMouseButtonDoubleClick(SelectedAsset);
}

void SBlueprintHeaderView::OnBlueprintChanged(UBlueprint* InBlueprint)
{
	RepopulateListView();
}

void SBlueprintHeaderView::OnStructChanged(UUserDefinedStruct* InStruct)
{
	RepopulateListView();
}

void SBlueprintHeaderView::OnCopy() const
{
	const int32 StringReserveSize = 2048;
	FString CopyText;
	CopyText.Reserve(StringReserveSize);
	for (const FHeaderViewListItemPtr& Item : ListItems)
	{
		if (ListView->IsItemSelected(Item))
		{
			CopyText += Item->GetRawItemString() + LINE_TERMINATOR;
		}
	}

	FPlatformApplicationMisc::ClipboardCopy(*CopyText);
}

bool SBlueprintHeaderView::CanCopy() const
{
	return ListView->GetNumItemsSelected() > 0;
}

void SBlueprintHeaderView::OnSelectAll()
{
	ListView->SetItemSelection(ListItems, true);
}

FReply SBlueprintHeaderView::BrowseToAssetClicked() const
{
	TArray<FAssetData> AssetsToSync;
	if (UBlueprint* Blueprint = SelectedBlueprint.Get())
	{
		AssetsToSync.Emplace(Blueprint);
	}
	else if (UUserDefinedStruct* Struct = SelectedStruct.Get())
	{
		AssetsToSync.Emplace(Struct);
	}

	if (!AssetsToSync.IsEmpty())
	{
		GEditor->SyncBrowserToObjects(AssetsToSync);
	}

	return FReply::Handled();
}

FReply SBlueprintHeaderView::OpenAssetEditorClicked() const
{
	if (UBlueprint* Blueprint = SelectedBlueprint.Get())
	{
		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(Blueprint);
	}
	else if (UUserDefinedStruct* Struct = SelectedStruct.Get())
	{
		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(Struct);
	}

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
