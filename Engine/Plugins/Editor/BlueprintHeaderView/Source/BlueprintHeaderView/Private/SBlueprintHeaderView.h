// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Misc/NotifyHook.h"

class FMenuBuilder;
enum class EValidatorResult;
template <typename ItemType> class SListView;

class SComboButton;
class UBlueprint;
class UUserDefinedStruct;
class UStruct;
class ITableRow;
class STableViewBase;
class FUICommandList;
class UEdGraph;

DECLARE_LOG_CATEGORY_EXTERN(LogBlueprintHeaderView, Log, All)

/** rich text decorators for BlueprintHeaderView Syntax Highlighting */
namespace HeaderViewSyntaxDecorators
{
	extern const FString CommentDecorator;
	extern const FString ErrorDecorator;
	extern const FString IdentifierDecorator;
	extern const FString KeywordDecorator;
	extern const FString MacroDecorator;
	extern const FString TypenameDecorator;
}

/** A base class for List Items in the Header View */
struct FHeaderViewListItem : public TSharedFromThis<FHeaderViewListItem>
{
	virtual ~FHeaderViewListItem() = default;

	/** Creates the widget for this list item */
	TSharedRef<SWidget> GenerateWidgetForItem();

	/** Creates a basic list item containing some text */
	static TSharedPtr<FHeaderViewListItem> Create(FString InRawString, FString InRichText);

	/** Returns the raw item text for copy actions */
	const FString& GetRawItemString() const { return RawItemString; }

	/** Allows the item to add items to the context menu if it is the only item selected */
	virtual void ExtendContextMenu(FMenuBuilder& InMenuBuilder, TWeakObjectPtr<UObject> InAsset) {}

	/** Called when this List Item is double clicked */
	virtual void OnMouseButtonDoubleClick(TWeakObjectPtr<UObject> InAsset) {};

protected:
	/** Empty base constructor hidden from public */
	FHeaderViewListItem() {};

	FHeaderViewListItem(FString&& InRawString, FString&& InRichText);

	/** 
	 * Formats a string into a C++ comment
	 * @param InComment The string to format as a C++ comment
	 * @param OutRawString The string formatted as a C++ comment
	 * @param OutRichString The string formatted as a C++ comment with rich text decorators for syntax highlighting
	 */
	static void FormatCommentString(FString InComment, FString& OutRawString, FString& OutRichString);

	/** 
	 * returns a string representing the full C++ typename for the given property, 
	 * including template params for container types
	 */
	static FString GetCPPTypenameForProperty(const FProperty* InProperty, bool bIsMemberProperty = false);

	/** Checks whether a string is a valid C++ identifier */
	static bool IsValidCPPIdentifier(const FString& InIdentifier);

	/** Returns the correct Error text for a Validator Result */
	static const FText& GetErrorTextFromValidatorResult(EValidatorResult Result);

protected:
	/** A rich text representation of the item, including syntax highlighting and errors */
	FString RichTextString;

	/** A raw string representation of the item, used for copying the item */
	FString RawItemString;

	/** Validation Error text for when a new name is not a valid C++ Identifier */
	static const FText InvalidCPPIdentifierErrorText;
};

using FHeaderViewListItemPtr = TSharedPtr<FHeaderViewListItem>;

class SBlueprintHeaderView : public SCompoundWidget, public FNotifyHook
{
public:
	SLATE_BEGIN_ARGS(SBlueprintHeaderView)
	{
	}

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	~SBlueprintHeaderView();

	//~ SWidget interface
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	//~ End SWidget interface

	//~ FNotifyHook interface
		/** Handles when the settings have changed, saves to config */
	virtual void NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged) override;
	//~ End FNotifyHook interface

	/** Callback for class picker menu selecting a blueprint asset */
	void OnAssetSelected(const FAssetData& SelectedAsset);
private:
	/** Gets the text for the class picker combo button */
	FText GetClassPickerText() const;

	/** Constructs a Blueprint Class picker menu widget */
	TSharedRef<SWidget> GetClassPickerMenuContent();

	/** Constructs a DetailsView widget for the settings menu */
	TSharedRef<SWidget> GetSettingsMenuContent();

	/** Generates a row for a given List Item */
	TSharedRef<ITableRow> GenerateRowForItem(FHeaderViewListItemPtr Item, const TSharedRef<STableViewBase>& OwnerTable) const;
	
	/** Clears the list and repopulates it with info for the selected Blueprint */
	void RepopulateListView();

	/** Adds items to the list view representing all functions present in the given blueprint */
	void PopulateFunctionItems(const UBlueprint* Blueprint);

	/** Gathers all function graphs from the blueprint and sorts them according to the selected method from config */
	void GatherFunctionGraphs(const UBlueprint* Blueprint, TArray<const UEdGraph*>& OutFunctionGraphs);

	/** Adds items to the list view representing all variables present in the given asset */
	void PopulateVariableItems(const UStruct* Struct);
	void AddVariableItems(TArray<const FProperty*> VarProperties);
	
	/** Sorts an array of properties to optimize for minimal C++ struct padding */
	void SortPropertiesForPadding(TArray<const FProperty*>& InOutProperties);

	/** Creates a context menu for the list view */
	TSharedPtr<SWidget> OnContextMenuOpening();

	/** Called when a List Item is double clicked */
	void OnItemDoubleClicked(FHeaderViewListItemPtr Item);

	/** Callback for when the selected asset is modified */
	void OnBlueprintChanged(UBlueprint* InBlueprint);
	void OnStructChanged(UUserDefinedStruct* InStruct);

	/** UI Command Functions */
	void OnCopy() const;
	bool CanCopy() const;
	void OnSelectAll();
	FReply BrowseToAssetClicked() const;
	FReply OpenAssetEditorClicked() const;
private:
	/** List of UI Commands for this scope */
	TSharedPtr<FUICommandList> CommandList;

	/** The asset currently being displayed by the header view */
	TWeakObjectPtr<UBlueprint> SelectedBlueprint;
	TWeakObjectPtr<UUserDefinedStruct> SelectedStruct;

	/** Reference to the Class Picker combo button widget */
	TSharedPtr<SComboButton> ClassPickerComboButton;

	/** Reference to the ListView Widget*/
	TSharedPtr<SListView<FHeaderViewListItemPtr>> ListView;

	/** List Items source */
	TArray<FHeaderViewListItemPtr> ListItems;
};
