// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/IDelegateInstance.h"
#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"

struct FAssetData;
struct FTableRowStyle;
struct FTextBlockStyle;

class FExtender;
class FSlateStyleSet;

class FBlueprintHeaderViewModule : public IModuleInterface
{
public:

	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	/** Returns whether the Header View supports the given class */
	static bool IsClassHeaderViewSupported(const UClass* InClass);

	static void OpenHeaderViewForAsset(FAssetData InAssetData);
private:
	void SetupAssetEditorMenuExtender();

	void SetupContentBrowserContextMenuExtender();

	static TSharedRef<FExtender> OnExtendContentBrowserAssetSelectionMenu(const TArray<FAssetData>& SelectedAssets);

public:
	/** Text style for the Header View Output */
	static FTextBlockStyle HeaderViewTextStyle;

	/** TableRow style for the Header View Output */
	static FTableRowStyle HeaderViewTableRowStyle;

	/** Style set for the header view */
	static TSharedPtr<FSlateStyleSet> HeaderViewStyleSet;

private: 
	/** Handle to our delegate so we can remove it at module shutdown */
	FDelegateHandle ContentBrowserExtenderDelegateHandle;
};
