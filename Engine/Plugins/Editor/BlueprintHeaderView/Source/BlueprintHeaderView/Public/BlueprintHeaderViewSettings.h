// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "BlueprintHeaderViewSettings.generated.h"

#define UE_API BLUEPRINTHEADERVIEW_API

UENUM()
enum class EHeaderViewSortMethod : uint8
{
	// Properties will stay in the same order they were in the Blueprint class.
	None, 

	// Properties will be grouped together by Access Specifiers, in order of visibility (public, protected, private).
	SortByAccessSpecifier,
	
	// Properties will be sorted to minimize padding in compiled class layout.
	SortForOptimalPadding 
};

USTRUCT(NotBlueprintable)
struct FHeaderViewSyntaxColors
{
	GENERATED_BODY()

	FHeaderViewSyntaxColors();

public:
	UPROPERTY(config, EditAnywhere, Category="Settings")
	FLinearColor Comment;
	
	UPROPERTY(config, EditAnywhere, Category="Settings")
	FLinearColor Error;
	
	UPROPERTY(config, EditAnywhere, Category="Settings")
	FLinearColor Macro;
	
	UPROPERTY(config, EditAnywhere, Category="Settings")
	FLinearColor Typename;
	
	UPROPERTY(config, EditAnywhere, Category="Settings")
	FLinearColor Identifier;
	
	UPROPERTY(config, EditAnywhere, Category="Settings")
	FLinearColor Keyword;
	
};

/** Settings for the Blueprint Header View Plugin */
UCLASS(MinimalAPI, config = EditorPerProjectUserSettings)
class UBlueprintHeaderViewSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UE_API UBlueprintHeaderViewSettings();

	//~ Begin UDeveloperSettings interface
	UE_API virtual FName GetCategoryName() const override;
	UE_API virtual FText GetSectionText() const override;
	UE_API virtual FName GetSectionName() const override;
	//~ End UDeveloperSettings interface

protected:
	//~ Begin UObject interface
	UE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End UObject interface

public:

	/** Syntax Highlighting colors for Blueprint Header View output */
	UPROPERTY(config, EditAnywhere, Category="Settings|Style")
	FHeaderViewSyntaxColors SyntaxColors;
	
	/** Highlight color for selected items in the Blueprint Header View output */
	UPROPERTY(config, EditAnywhere, Category="Settings|Style")
	FLinearColor SelectionColor = FLinearColor(0.3f, 0.3f, 1.0f, 1.0f);

	/** Font Size for the Blueprint Header View output */
	UPROPERTY(config, EditAnywhere, Category="Settings|Style", meta=(ClampMin=6, ClampMax=72))
	int32 FontSize = 9;
	
	/** Sorting Method for Header View Functions and Properties */
	UPROPERTY(config, EditAnywhere, Category="Settings")
	EHeaderViewSortMethod SortMethod = EHeaderViewSortMethod::None;
};

#undef UE_API
